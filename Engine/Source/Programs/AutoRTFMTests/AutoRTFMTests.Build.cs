// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AutoRTFMTests : ModuleRules
{
	public AutoRTFMTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AutoRTFM",
				"Catch2Extras",
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Projects",
			}
		);

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] { "DesktopPlatform" }
			);
		}

		PrivateIncludePaths.AddRange(
			new string[]
			{
				Path.Combine(GetModuleDirectory("AutoRTFM"), "Private"),
				Path.Combine(GetModuleDirectory("Core"), "Private"),
				Path.Combine(GetModuleDirectory("CoreUObject"), "Internal"),
			});

		PCHUsage = PCHUsageMode.NoPCHs;
		FPSemantics = FPSemanticsMode.Precise;
	}
}
