// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;
using System.Collections.Generic;
using UnrealBuildBase;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AutoRTFMTestsWithExceptionsTarget : AutoRTFMTestsTarget
{
	public AutoRTFMTestsWithExceptionsTarget(TargetInfo Target) : base(Target)
	{
		bForceEnableExceptions = true;
	}
}
