// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;
using System.Collections.Generic;
using UnrealBuildBase;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AutoRTFMTestsWithICUTarget : AutoRTFMTestsTarget
{
	public AutoRTFMTestsWithICUTarget(TargetInfo Target) : base(Target)
	{
		// Editor uses ICU so we need to test with it too.
		bCompileICU = true;
	}
}
