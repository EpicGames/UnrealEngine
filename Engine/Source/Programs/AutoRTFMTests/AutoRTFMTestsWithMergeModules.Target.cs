// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;
using System.Collections.Generic;
using UnrealBuildBase;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AutoRTFMTestsWithMergeModulesTarget : AutoRTFMTestsTarget
{
	public AutoRTFMTestsWithMergeModulesTarget(TargetInfo Target) : base(Target)
	{
		bMergeModules = true;
	}
}
