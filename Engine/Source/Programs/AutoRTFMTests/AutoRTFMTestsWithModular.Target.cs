// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;
using System.Collections.Generic;
using UnrealBuildBase;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AutoRTFMTestsWithModularTarget : AutoRTFMTestsTarget
{
	public AutoRTFMTestsWithModularTarget(TargetInfo Target) : base(Target)
	{
		LinkType = TargetLinkType.Modular;
		GlobalDefinitions.Add("AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES=1");
	}
}
