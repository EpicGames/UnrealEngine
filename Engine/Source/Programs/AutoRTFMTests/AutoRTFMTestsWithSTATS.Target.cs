// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;
using System.Collections.Generic;
using UnrealBuildBase;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AutoRTFMTestsWithSTATSTarget : AutoRTFMTestsTarget
{
	public AutoRTFMTestsWithSTATSTarget(TargetInfo Target) : base(Target)
	{
		GlobalDefinitions.Remove("ENABLE_STATNAMEDEVENTS=1");
		GlobalDefinitions.Remove("ENABLE_STATNAMEDEVENTS_UOBJECT=1");
		GlobalDefinitions.Add("FORCE_USE_STATS=1");
	}
}
