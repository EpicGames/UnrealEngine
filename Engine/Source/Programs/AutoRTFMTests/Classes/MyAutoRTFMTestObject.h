// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MyAutoRTFMTestObject.generated.h"

UCLASS()
class UMyAutoRTFMTestObject : public UObject
{
	GENERATED_BODY()

public:
	using FConstructorCallback = void(const FObjectInitializer&, UMyAutoRTFMTestObject&);

	UMyAutoRTFMTestObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	int Value = 42;

	static FConstructorCallback* ConstructorCallback;

	void DoNothing() const {}
};
