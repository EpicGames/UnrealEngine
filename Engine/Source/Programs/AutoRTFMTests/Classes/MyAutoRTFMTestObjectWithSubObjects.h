// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MyAutoRTFMTestObjectWithSubObjects.generated.h"

UCLASS()
class UMyAutoRTFMTestObjectWithSubObjects : public UObject
{
	GENERATED_BODY()

public:
	static FName GetSubObjectName();

	UMyAutoRTFMTestObjectWithSubObjects(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UObject* SubObject;

	void DoNothing() const {}
};
