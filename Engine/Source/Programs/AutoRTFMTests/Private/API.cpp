// Copyright Epic Games, Inc. All Rights Reserved.

#include "API.h"

#include "AutoRTFM.h"
#include "AutoRTFMTestUtils.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

TEST_CASE("API.autortfm_is_closed")
{
    REQUIRE(false == autortfm_is_closed());

    // Set to the opposite of what we expect at the end of function.
    bool bInTransaction = false;
    bool bInOpenNest = true;
    bool bInClosedNestInOpenNest = false;

    AutoRTFM::Testing::Commit([&]
    {
        bInTransaction = autortfm_is_closed();

        AutoRTFM::Open([&]
        {
            bInOpenNest = autortfm_is_closed();

            REQUIRE(AutoRTFM::ETransactionStatus::Executing == AutoRTFM::Close([&]
            {
                bInClosedNestInOpenNest = autortfm_is_closed();
            }));
        });
    });

    REQUIRE(true == bInTransaction);
    REQUIRE(false == bInOpenNest);
    REQUIRE(true == bInClosedNestInOpenNest);
}

TEST_CASE("API.autortfm_abort_transaction")
{
    bool bBeforeNest = false;
    bool bInNest = false;
    bool bAfterNest = false;
    
    AutoRTFM::Testing::Commit([&]
    {
        bBeforeNest = true;

        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            // Because we are aborting this won't actually occur!
            bInNest = true;

            autortfm_abort_transaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        bAfterNest = true;
    });

    REQUIRE(true == bBeforeNest);
    REQUIRE(false == bInNest);
    REQUIRE(true == bAfterNest);
}

TEST_CASE("API.autortfm_open")
{
    int Answer = 6 * 9;

    // An open call outside a transaction succeeds.
    autortfm_open([](void* Arg)
    {
        *static_cast<int*>(Arg) = 42;
    }, &Answer);

    REQUIRE(42 == Answer);

    REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
    {
        // An open call inside a transaction succeeds also.
        autortfm_open([](void* Arg)
        {
            *static_cast<int*>(Arg) *= 2;
        }, &Answer);

        AutoRTFM::AbortTransaction();
    }));

    REQUIRE(84 == Answer);
}

TEST_CASE("API.autortfm_register_open_to_closed_functions")
{
    static autortfm_open_to_closed_mapping Mappings[] = 
    {
        {
            reinterpret_cast<void*>(NoAutoRTFM::DoSomethingC),
            reinterpret_cast<void*>(NoAutoRTFM::DoSomethingInTransactionC),
        },
        { nullptr, nullptr },
    };
    
    static autortfm_open_to_closed_table Table;
    Table.Mappings = Mappings;

    autortfm_register_open_to_closed_functions(&Table);

    int I = -42;

    AutoRTFM::Testing::Commit([&]
    {
        I = NoAutoRTFM::DoSomethingC(I);
    });

    REQUIRE(0 == I);
}

TEST_CASE("API.autortfm_on_commit")
{
    bool bOuterTransaction = false;
    bool bInnerTransaction = false;
    bool bInnerTransactionWithAbort = false;
    bool bInnerOpenNest = false;

    AutoRTFM::Testing::Commit([&]
    {
        autortfm_on_commit([](void* const Arg)
        {
            *static_cast<bool* const>(Arg) = true;
        }, &bOuterTransaction);

        // This should only be modified on the commit!
        if (bOuterTransaction)
        {
            AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Testing::Commit([&]
        {
            autortfm_on_commit([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerTransaction);
        });

        // This should only be modified on the commit!
        if (bInnerTransaction)
        {
            AutoRTFM::AbortTransaction();
        }

        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            autortfm_on_commit([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerTransactionWithAbort);

            AutoRTFM::AbortTransaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        // This should never be modified because its transaction aborted!
        if (bInnerTransactionWithAbort)
        {
            AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Open([&]
        {
            autortfm_on_commit([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerOpenNest);

            // This should be modified immediately!
            if (!bInnerOpenNest)
            {
                AutoRTFM::AbortTransaction();
            }
        });
    });

    REQUIRE(true == bOuterTransaction);
    REQUIRE(true == bInnerTransaction);
    REQUIRE(false == bInnerTransactionWithAbort);
    REQUIRE(true == bInnerOpenNest);
}

TEST_CASE("API.autortfm_on_abort")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

    bool bOuterTransaction = false;
    bool bInnerTransaction = false;
    bool bInnerTransactionWithAbort = false;
    bool bInnerOpenNest = false;

    AutoRTFM::Testing::Commit([&]
    {
        autortfm_on_abort([](void* const Arg)
        {
            *static_cast<bool* const>(Arg) = true;
        }, &bOuterTransaction);

        AutoRTFM::Testing::Commit([&]
        {
            autortfm_on_abort([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerTransaction);
        });

        AutoRTFM::Testing::Abort([&]
        {
            autortfm_on_abort([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerTransactionWithAbort);

            AutoRTFM::AbortTransaction();
        });

        AutoRTFM::Open([&]
        {
            autortfm_on_abort([](void* const Arg)
            {
                *static_cast<bool* const>(Arg) = true;
            }, &bInnerOpenNest);
        });
    });

    REQUIRE(false == bOuterTransaction);
    REQUIRE(false == bInnerTransaction);
    REQUIRE(true == bInnerTransactionWithAbort);
    REQUIRE(false == bInnerOpenNest);
}

TEST_CASE("API.autortfm_did_allocate")
{
    constexpr unsigned Size = 1024;
    unsigned BumpAllocator[Size];
    unsigned NextBump = 0;

    AutoRTFM::Testing::Commit([&]
    {
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnAbort([&]
		{
			NextBump = 0;
		});

        for (unsigned I = 0; I < Size; I++)
        {
            unsigned* Data;
            AutoRTFM::Open([&]
            {
                Data = reinterpret_cast<unsigned*>(autortfm_did_allocate(
                    &BumpAllocator[NextBump++],
                    sizeof(unsigned)));
            });

            *Data = I;
        }
    });

    for (unsigned I = 0; I < Size; I++)
    {
        REQUIRE(I == BumpAllocator[I]);
    }
}

TEST_CASE("API.autortfm_unreachable")
{
	AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
	AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2(false);

	bool bTouched = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;
		autortfm_unreachable("Oh noes!");
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
	REQUIRE(false == bTouched);
}

TEST_CASE("API.ETransactionResult")
{
    int Answer = 6 * 9;

    REQUIRE(AutoRTFM::ETransactionResult::Committed == AutoRTFM::Transact([&]
    {
        Answer = 42;
    }));

    REQUIRE(42 == Answer);

    REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
    {
        Answer = 13;
        AutoRTFM::AbortTransaction();
    }));

    REQUIRE(42 == Answer);
}

TEST_CASE("API.IsTransactional")
{
    REQUIRE(false == AutoRTFM::IsTransactional());

    bool bInTransaction = false;
    bool bInOpenNest = false;
	bool bInAbort = true;
	bool bInCommit = true;
	bool bInComplete = true;

    AutoRTFM::Testing::Commit([&]
    {
        bInTransaction = AutoRTFM::IsTransactional();

        AutoRTFM::Open([&]
        {
            bInOpenNest = AutoRTFM::IsTransactional();
        });

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&]
			{
				bInAbort = AutoRTFM::IsTransactional();
			});

			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::OnCommit([&]
		{
			bInCommit = AutoRTFM::IsTransactional();
		});

		AutoRTFM::OnComplete([&]
		{
			bInComplete = AutoRTFM::IsTransactional();
		});
    });

    REQUIRE(true == bInTransaction);
    REQUIRE(true == bInOpenNest);
	REQUIRE(false == bInAbort);
	REQUIRE(false == bInCommit);
	REQUIRE(false == bInComplete);
}

TEST_CASE("API.IsClosed")
{
    REQUIRE(false == AutoRTFM::IsClosed());

    // Set to the opposite of what we expect at the end of function.
    bool bInTransaction = false;
    bool bInOpenNest = true;
    bool bInClosedNestInOpenNest = false;
	bool bInAbort = true;
	bool bInCommit = true;
	bool bInComplete = true;

    AutoRTFM::Testing::Commit([&]
    {
        bInTransaction = AutoRTFM::IsClosed();

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&]
			{
				bInAbort = AutoRTFM::IsClosed();
			});

			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::OnCommit([&]
		{
			bInCommit = AutoRTFM::IsClosed();
		});

		AutoRTFM::OnComplete([&]
		{
			bInComplete = AutoRTFM::IsClosed();
		});

        AutoRTFM::Open([&]
        {
            bInOpenNest = AutoRTFM::IsClosed();

            AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
            {
                bInClosedNestInOpenNest = AutoRTFM::IsClosed();
            });
			REQUIRE(Status == AutoRTFM::ETransactionStatus::Executing);
        });
    });

    REQUIRE(true == bInTransaction);
    REQUIRE(false == bInOpenNest);
    REQUIRE(true == bInClosedNestInOpenNest);
	REQUIRE(false == bInAbort);
	REQUIRE(false == bInCommit);
	REQUIRE(false == bInComplete);
}

TEST_CASE("API.IsCommittingOrAborting")
{
	REQUIRE(false == AutoRTFM::IsCommittingOrAborting());

	// Set to the opposite of what we expect at the end of function.
	bool bInTransaction = true;
	bool bInOpenNest = true;
	bool bInClosedNestInOpenNest = true;
	bool bInAbort = false;
	bool bInCommit = false;
	bool bInComplete = false;
	bool bInRetryAbort = false;
	bool bInRetryComplete = false;
	bool bInRetryRetry = false;

	AutoRTFM::Testing::Commit([&]
	{
		bInTransaction = AutoRTFM::IsCommittingOrAborting();

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&]
			{
				bInAbort = AutoRTFM::IsCommittingOrAborting();
			});

			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::OnCommit([&]
		{
			bInCommit = AutoRTFM::IsCommittingOrAborting();
		});

		AutoRTFM::OnComplete([&]
		{
			bInComplete = AutoRTFM::IsCommittingOrAborting();
		});

		AutoRTFM::Open([&]
		{
			bInOpenNest = AutoRTFM::IsCommittingOrAborting();

			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				bInClosedNestInOpenNest = AutoRTFM::IsCommittingOrAborting();
			});

			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);
		});
	});

	bool bAlreadyRetried = false;

	AutoRTFM::Testing::Commit([&]
	{
		if (!bAlreadyRetried)
		{
			AutoRTFM::OnAbort([&]
			{
				bInRetryAbort = AutoRTFM::IsCommittingOrAborting();
			});
			AutoRTFM::OnAbort([&]
			{
				bInRetryComplete = AutoRTFM::IsCommittingOrAborting();
			});
			AutoRTFM::OnRetry([&]
			{
				bInRetryRetry = AutoRTFM::IsCommittingOrAborting();
				bAlreadyRetried = true;
			});

			AutoRTFM::CascadingRetryTransaction();
		}
	});

	REQUIRE(false == bInTransaction);
	REQUIRE(false == bInOpenNest);
	REQUIRE(false == bInClosedNestInOpenNest);
	REQUIRE(true == bInAbort);
	REQUIRE(true == bInCommit);
	REQUIRE(true == bInComplete);
	REQUIRE(true == bInRetryAbort);
	REQUIRE(true == bInRetryComplete);
	REQUIRE(true == bInRetryRetry);
}

TEST_CASE("API.OnRetry")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

	enum EEvent
	{
		OnStartTransaction,
		OnEndTransaction,
		OnAbort1,
		OnAbort2,
		OnCommit1,
		OnCommit2,
		OnComplete1,
		OnComplete2,
		OnRetry1,
		OnRetry2,
		OnAbortDestructor1,
		OnAbortDestructor2,
		OnCommitDestructor1,
		OnCommitDestructor2,
		OnCompleteDestructor1,
		OnCompleteDestructor2,
		OnRetryDestructor1,
		OnRetryDestructor2,
	};

	enum EMode
	{
		Commit,
		Abort,
		Retry,
	};

	auto Run = [] (EMode Mode) -> std::vector<EEvent>
	{
		std::vector<EEvent> Events;
		auto AddEvent = [&] AUTORTFM_OPEN (EEvent Event){
			Events.push_back(Event);
		};
		struct AUTORTFM_ENABLE FAddEventOnDestruction
		{
			std::vector<EEvent>& Events;
			EEvent Event;
			mutable bool bMovedOrCopied = false;
			FAddEventOnDestruction(std::vector<EEvent>& Events, EEvent Event) : Events{Events}, Event{Event} {}
			FAddEventOnDestruction(const FAddEventOnDestruction& Other) : Events{Other.Events}, Event{Other.Event}
			{
				Other.bMovedOrCopied = true;
			}
			FAddEventOnDestruction(FAddEventOnDestruction&& Other) : Events{Other.Events}, Event{Other.Event}
			{
				Other.bMovedOrCopied = true;
			}
			FAddEventOnDestruction& operator = (const FAddEventOnDestruction&) = delete;
			FAddEventOnDestruction& operator = (FAddEventOnDestruction&&) = delete;

			~FAddEventOnDestruction()
			{
				if (!bMovedOrCopied)
				{
					AutoRTFM::Open([&] { Events.push_back(Event); });
				}
			}
		};

		bool bHasRetried = false;
	
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			AddEvent(OnStartTransaction);

			AutoRTFM::OnAbort([&, Dtor = FAddEventOnDestruction{Events, OnAbortDestructor1}]
			{
				AddEvent(OnAbort1);
			});

			AutoRTFM::OnCommit([&, Dtor = FAddEventOnDestruction{Events, OnCommitDestructor1}]
			{
				AddEvent(OnCommit1);
			});

			AutoRTFM::OnComplete([&, Dtor = FAddEventOnDestruction{Events, OnCompleteDestructor1}]
			{
				AddEvent(OnComplete1);
			});

			AutoRTFM::OnRetry([&, Dtor = FAddEventOnDestruction{Events, OnRetryDestructor1}]
			{
				AddEvent(OnRetry1);
				bHasRetried = true;
			});

			AutoRTFM::OnAbort([&, Dtor = FAddEventOnDestruction{Events, OnAbortDestructor2}]
			{
				AddEvent(OnAbort2);
			});

			AutoRTFM::OnCommit([&, Dtor = FAddEventOnDestruction{Events, OnCommitDestructor2}]
			{
				AddEvent(OnCommit2);
			});

			AutoRTFM::OnComplete([&, Dtor = FAddEventOnDestruction{Events, OnCompleteDestructor2}]
			{
				AddEvent(OnComplete2);
			});

			AutoRTFM::OnRetry([&, Dtor = FAddEventOnDestruction{Events, OnRetryDestructor2}]
			{
				AddEvent(OnRetry2);
				bHasRetried = true;
			});

			switch (Mode)
			{
				case Commit:
					break;
				case Abort:
					AutoRTFM::AbortTransaction();
					break;
				case Retry:
					if (!bHasRetried)
					{
						AutoRTFM::CascadingRetryTransaction();
					}
					break;
			}
			
			AddEvent(OnEndTransaction);
		});
		
		switch (Mode)
		{
			case Commit:
				REQUIRE(Result == AutoRTFM::ETransactionResult::Committed);
				break;
			case Abort:
				REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByRequest);
				break;
			case Retry:
				REQUIRE(Result == AutoRTFM::ETransactionResult::Committed);
				break;
		}
		
		return Events;
	};

	SECTION("Commit")
	{
		std::vector<EEvent> Events = Run(Commit);
		REQUIRE(Events == std::vector<EEvent>{
			OnStartTransaction,
			OnEndTransaction,
			OnAbortDestructor1,
			OnAbortDestructor2,
			OnCommit1,
			OnCommitDestructor1,
			OnCommit2,
			OnCommitDestructor2,
			OnRetryDestructor1,
			OnRetryDestructor2,
			OnComplete1,
			OnCompleteDestructor1,
			OnComplete2,
			OnCompleteDestructor2,
		});
	}

	SECTION("Abort")
	{
		std::vector<EEvent> Events = Run(Abort);
		REQUIRE(Events == std::vector<EEvent>{
			OnStartTransaction,
			OnCommitDestructor1,
			OnCommitDestructor2,
			OnAbort2,
			OnAbortDestructor2,
			OnAbort1,
			OnAbortDestructor1,
			OnRetryDestructor1,
			OnRetryDestructor2,
			OnComplete1,
			OnCompleteDestructor1,
			OnComplete2,
			OnCompleteDestructor2,
		});
	}

	SECTION("Retry")
	{
		std::vector<EEvent> Events = Run(Retry);
		REQUIRE(Events == std::vector<EEvent>{
			OnStartTransaction,
			OnCommitDestructor1,
			OnCommitDestructor2,
			OnAbort2,
			OnAbortDestructor2,
			OnAbort1,
			OnAbortDestructor1,
			OnCompleteDestructor1,
			OnCompleteDestructor2,
			OnRetry1,
			OnRetryDestructor1,
			OnRetry2,
			OnRetryDestructor2,
			// Retry
			OnStartTransaction,
			OnEndTransaction,
			OnAbortDestructor1,
			OnAbortDestructor2,
			OnCommit1,
			OnCommitDestructor1,
			OnCommit2,
			OnCommitDestructor2,
			OnRetryDestructor1,
			OnRetryDestructor2,
			OnComplete1,
			OnCompleteDestructor1,
			OnComplete2,
			OnCompleteDestructor2,
		});
	}
}

TEST_CASE("API.Transact")
{
	int Answer = 6 * 9;

	REQUIRE(AutoRTFM::ETransactionResult::Committed == AutoRTFM::Transact([&]
	{
		Answer = 42;
	}));

	REQUIRE(42 == Answer);
}

TEST_CASE("API.TransactMacro_NoAbort")
{
	int Answer = 6 * 9;

	// Allowing the transaction to commit should work.
	UE_AUTORTFM_TRANSACT
	{
		Answer = 42;
	};

	REQUIRE(42 == Answer);
}

TEST_CASE("API.TransactMacro_WithAbort")
{
	int Answer = 42;

	// Aborting the transaction should also work.
	UE_AUTORTFM_TRANSACT
	{
		Answer = 6 * 9;
		AutoRTFM::AbortTransaction();
	};

	REQUIRE(42 == Answer);
}

TEST_CASE("API.Commit")
{
    int Answer = 6 * 9;

    AutoRTFM::Testing::Commit([&]
    {
        Answer = 42;
    });

    REQUIRE(42 == Answer);
}

TEST_CASE("API.Abort")
{
    bool bBeforeNest = false;
    bool bInNest = false;
    bool bAfterNest = false;

    AutoRTFM::Testing::Commit([&]
    {
        bBeforeNest = true;

        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            // Because we are aborting this won't actually occur!
            bInNest = true;

            AutoRTFM::AbortTransaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        bAfterNest = true;
    });

    REQUIRE(true == bBeforeNest);
    REQUIRE(false == bInNest);
    REQUIRE(true == bAfterNest);
}

TEST_CASE("API.Open")
{
    int Answer = 6 * 9;

    // An open call outside a transaction succeeds.
    AutoRTFM::Open([&]
    {
        Answer = 42;
    });

    REQUIRE(42 == Answer);

    REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
    {
        // An open call inside a transaction succeeds also.
        AutoRTFM::Open([&]
        {
            Answer *= 2;
        });

        AutoRTFM::AbortTransaction();
    }));

    REQUIRE(84 == Answer);
}

TEST_CASE("API.OpenMacro_NoAbort")
{
    int Answer = 6 * 9;

    // An open call outside a transaction succeeds.
    UE_AUTORTFM_OPEN
    {
        Answer = 42;
    };

    REQUIRE(42 == Answer);
}

TEST_CASE("API.OpenMacro_WithAbort")
{
	int Answer = 21;

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
    {
        // An open call inside a transaction succeeds.
        UE_AUTORTFM_OPEN
        {
            Answer *= 2;
        };

        AutoRTFM::AbortTransaction();
    }));

    REQUIRE(42 == Answer);
}

TEST_CASE("API.Close")
{
    bool bInClosedNest = false;
    bool bInOpenNest = false;
    bool bInClosedNestInOpenNest = false;

    AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
    {
        // A closed call inside a transaction does not abort.
        AutoRTFM::ETransactionStatus CloseStatusA = AutoRTFM::Close([&] { bInClosedNest = true; });
        REQUIRE(AutoRTFM::ETransactionStatus::Executing == CloseStatusA);

        AutoRTFM::Open([&]
        {
            AutoRTFM::ETransactionStatus CloseStatusB = AutoRTFM::Close([&]
            {
                bInClosedNestInOpenNest = true;
            });
            REQUIRE(AutoRTFM::ETransactionStatus::Executing == CloseStatusB);

            bInOpenNest = true;
        });

        AutoRTFM::AbortTransaction();
    });

    REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

    REQUIRE(false == bInClosedNest);
    REQUIRE(true == bInOpenNest);
    REQUIRE(false == bInClosedNestInOpenNest);
}

TEST_CASE("API.OnCommit")
{
    bool bOuterTransaction = false;
    bool bInnerTransaction = false;
    bool bInnerTransactionWithAbort = false;
    bool bInnerOpenNest = false;

    REQUIRE(AutoRTFM::ETransactionResult::Committed == AutoRTFM::Transact([&]
    {
        AutoRTFM::OnCommit([&]
        {
            bOuterTransaction = true;
        });

        // This should only be modified on the commit!
        if (bOuterTransaction)
        {
            AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Testing::Commit([&]
        {
            AutoRTFM::OnCommit([&]
            {
                bInnerTransaction = true;
            });
        });

        // This should only be modified on the commit!
        if (bInnerTransaction)
        {
			AutoRTFM::AbortTransaction();
        }


        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            AutoRTFM::OnCommit([&]
            {
                bInnerTransactionWithAbort = true;
            });

            AutoRTFM::AbortTransaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        // This should never be modified because its transaction aborted!
        if (bInnerTransactionWithAbort)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Open([&]
        {
            AutoRTFM::OnCommit([&]
            {
                bInnerOpenNest = true;
            });

            // This should be modified immediately!
            if (!bInnerOpenNest)
            {
				AutoRTFM::AbortTransaction();
            }
        });
    }));

    REQUIRE(true == bOuterTransaction);
    REQUIRE(true == bInnerTransaction);
    REQUIRE(false == bInnerTransactionWithAbort);
    REQUIRE(true == bInnerOpenNest);
}

TEST_CASE("API.OnComplete")
{
	bool bTriggered = false;

	SECTION("Commit(OnComplete)")
	{
		AutoRTFM::Testing::Commit([&]
		{
			// We reset `bTriggered` at the top of each transaction, because our retry testing can 
			// cause us to rerun the transaction after `bTriggered` has been set via OnComplete.
			bTriggered = false;
			AutoRTFM::OnComplete([&]
			{
				bTriggered = true;
			});
			REQUIRE(!bTriggered);
		});
		REQUIRE(bTriggered);
	}

	SECTION("Abort(OnComplete)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			bTriggered = false;
			AutoRTFM::OnComplete([&]
			{
				bTriggered = true;
			});
			REQUIRE(!bTriggered);
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(bTriggered);
	}

	SECTION("Commit(Commit(OnComplete))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			bTriggered = false;
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::OnComplete([&]
				{
					bTriggered = true;
				});
				REQUIRE(!bTriggered);
			});
			REQUIRE(!bTriggered);
		});
		REQUIRE(bTriggered);
	}

	SECTION("Abort(Abort(OnComplete))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			bTriggered = false;
			AutoRTFM::Testing::Abort([&]
			{
				AutoRTFM::OnComplete([&]
				{
					bTriggered = true;
				});
				REQUIRE(!bTriggered);
				AutoRTFM::AbortTransaction();
			});
			REQUIRE(!bTriggered);
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(bTriggered);
	}

	SECTION("Commit(Abort(OnComplete))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			bTriggered = false;
			AutoRTFM::Testing::Abort([&]
			{
				AutoRTFM::OnComplete([&]
				{
					bTriggered = true;
				});
				REQUIRE(!bTriggered);
				AutoRTFM::AbortTransaction();
			});
			REQUIRE(!bTriggered);
		});
		REQUIRE(bTriggered);
	}

	SECTION("Abort(Commit(OnComplete))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			bTriggered = false;
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::OnComplete([&]
				{
					bTriggered = true;
				});
				REQUIRE(!bTriggered);
			});
			REQUIRE(!bTriggered);
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(bTriggered);
	}
}

TEST_CASE("API.OnCommit_MutableCapture")
{
	FString Message = "Hello";

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnCommit([MessageCopy = Message]() mutable
		{
			MessageCopy += " World!";
			REQUIRE(MessageCopy == "Hello World!");
		});
	});
}

TEST_CASE("API.OnComplete_MutableCapture")
{
	FString Message = "Hello";

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnComplete([MessageCopy = Message]() mutable
		{
			MessageCopy += " World!";
			REQUIRE(MessageCopy == "Hello World!");
		});
	});
}

TEST_CASE("API.OnCommitMacro_NoAbort")
{
	int Value = 123;

	UE_AUTORTFM_TRANSACT
	{
		UE_AUTORTFM_ONCOMMIT(&Value)
		{
			Value = 456;
		};

		Value = 789;
	};

	REQUIRE(Value == 456);
}

TEST_CASE("API.OnCommitMacro_WithAbort")
{
	int Value = 123;

	UE_AUTORTFM_TRANSACT
	{
		UE_AUTORTFM_ONCOMMIT(&Value)
		{
			Value = 456;
		};

		Value = 789;

		AutoRTFM::AbortTransaction();
	};

	REQUIRE(Value == 123);
}

TEST_CASE("API.OnAbort")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

    bool bOuterTransaction = false;
    bool bInnerTransaction = false;
    bool bInnerTransactionWithAbort = false;
    bool bInnerOpenNest = false;

    REQUIRE(AutoRTFM::ETransactionResult::Committed == AutoRTFM::Transact([&]
    {
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnAbort([&]
		{
			bOuterTransaction = false;
			bInnerTransaction = false;
			bInnerTransactionWithAbort = false;
			bInnerOpenNest = false;
		});

        AutoRTFM::OnAbort([&]
        {
            bOuterTransaction = true;
        });

        // This should only be modified on the commit!
        if (bOuterTransaction)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Testing::Commit([&]
        {
            AutoRTFM::OnAbort([&]
            {
                bInnerTransaction = true;
            });
        });

        // This should only be modified on the commit!
        if (bInnerTransaction)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            AutoRTFM::OnAbort([&]
            {
                bInnerTransactionWithAbort = true;
            });

			AutoRTFM::AbortTransaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        // Inner OnAbort runs eagerly
        if (!bInnerTransactionWithAbort)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Open([&]
        {
            AutoRTFM::OnAbort([&]
            {
                bInnerOpenNest = true;
            });
        });

        // This should only be modified on the commit!
        if (bInnerOpenNest)
        {
			AutoRTFM::AbortTransaction();
        }
    }));

    REQUIRE(false == bOuterTransaction);
    REQUIRE(false == bInnerTransaction);
    REQUIRE(true == bInnerTransactionWithAbort);
    REQUIRE(false == bInnerOpenNest);
}

TEST_CASE("API.OnPreAbort")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

    bool bOuterTransaction = false;
    bool bInnerTransaction = false;
    bool bInnerTransactionWithAbort = false;
    bool bInnerOpenNest = false;

    REQUIRE(AutoRTFM::ETransactionResult::Committed == AutoRTFM::Transact([&]
    {
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnPreAbort([&]
		{
			bOuterTransaction = false;
			bInnerTransaction = false;
			bInnerTransactionWithAbort = false;
			bInnerOpenNest = false;
		});

        AutoRTFM::OnPreAbort([&]
        {
            bOuterTransaction = true;
        });

        // This should only be modified on the commit!
        if (bOuterTransaction)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Testing::Commit([&]
        {
            AutoRTFM::OnPreAbort([&]
            {
                bInnerTransaction = true;
            });
        });

        // This should only be modified on the commit!
        if (bInnerTransaction)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            AutoRTFM::OnPreAbort([&]
            {
                bInnerTransactionWithAbort = true;
            });

			AutoRTFM::AbortTransaction();
        });

        REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

        // Inner OnPreAbort runs eagerly
        if (!bInnerTransactionWithAbort)
        {
			AutoRTFM::AbortTransaction();
        }

        AutoRTFM::Open([&]
        {
            AutoRTFM::OnPreAbort([&]
            {
                bInnerOpenNest = true;
            });
        });

        // This should only be modified on the commit!
        if (bInnerOpenNest)
        {
			AutoRTFM::AbortTransaction();
        }
    }));

    REQUIRE(false == bOuterTransaction);
    REQUIRE(false == bInnerTransaction);
    REQUIRE(true == bInnerTransactionWithAbort);
    REQUIRE(false == bInnerOpenNest);
}

TEST_CASE("API.OnAbort_MutableCapture")
{
	FString Message = "Hello";

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnAbort([MessageCopy = Message]() mutable
		{
			MessageCopy += " World!";
			REQUIRE(MessageCopy == "Hello World!");
		});

		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("API.OnPreAbort_MutableCapture")
{
	FString Message = "Hello";

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnPreAbort([MessageCopy = Message]() mutable
		{
			MessageCopy += " World!";
			REQUIRE(MessageCopy == "Hello World!");
		});

		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("API.OnAbortMacro_NoAbort")
{
	int Value = 123;

	UE_AUTORTFM_TRANSACT
	{
		Value = 456;

		UE_AUTORTFM_ONABORT(&Value)
		{
			Value = 123;
		};
	};

	REQUIRE(Value == 456);
}

TEST_CASE("API.OnPreAbortMacro_NoAbort")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

	int Value = 123;
	bool bReachedOnPreAbort = false;

	UE_AUTORTFM_TRANSACT
	{
		Value = 456;

		UE_AUTORTFM_ONPREABORT(&Value, &bReachedOnPreAbort)
		{
			UE_AUTORTFM_OPEN
			{
				bReachedOnPreAbort = true;
			};
		};
	};

	REQUIRE(Value == 456);
	REQUIRE(!bReachedOnPreAbort);
}

TEST_CASE("API.OnAbortMacro_WithAbort")
{
	int Value = 123;

	UE_AUTORTFM_TRANSACT
	{
		Value = 234;

		UE_AUTORTFM_ONABORT(&Value) 
		{
			Value = 123;
		};

		AutoRTFM::AbortTransaction();
	};

	REQUIRE(Value == 123);
}

TEST_CASE("API.OnPreAbortMacro_WithAbort")
{
	int Value = 123;
	bool bReachedOnPreAbort = false;

	UE_AUTORTFM_TRANSACT
	{
		Value = 234;

		UE_AUTORTFM_ONPREABORT(&Value, &bReachedOnPreAbort)
		{
			// In OnPreAbort, the value is still changed.
			REQUIRE(Value == 234);
			UE_AUTORTFM_OPEN
			{
				bReachedOnPreAbort = true;
			};
		};

		UE_AUTORTFM_ONABORT(&Value)
		{
			// In OnAbort, the value has been rolled back.
			REQUIRE(Value == 123);
		};

		AutoRTFM::AbortTransaction();
	};

	REQUIRE(Value == 123);
	REQUIRE(bReachedOnPreAbort);
}

TEST_CASE("API.DidAllocate")
{
    constexpr unsigned Size = 1024;
    unsigned BumpAllocator[Size];
    unsigned NextBump = 0;

    AutoRTFM::Testing::Commit([&]
    {
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnAbort([&]
		{
			NextBump = 0;
		});

        for (unsigned I = 0; I < Size; I++)
        {
            unsigned* Data;
            AutoRTFM::Open([&]
            {
                Data = reinterpret_cast<unsigned*>(AutoRTFM::DidAllocate(
                    &BumpAllocator[NextBump++],
                    sizeof(unsigned)));
            });

            *Data = I;
        }
    });

    for (unsigned I = 0; I < Size; I++)
    {
        REQUIRE(I == BumpAllocator[I]);
    }
}

TEST_CASE("API.IsOnCurrentTransactionStack")
{
	{
		int OnStackNotInTransaction = 1;
		REQUIRE(!AutoRTFM::IsOnCurrentTransactionStack(&OnStackNotInTransaction));

		int* OnHeapNotInTransaction = new int{2};
		REQUIRE(!AutoRTFM::IsOnCurrentTransactionStack(OnHeapNotInTransaction));
		delete OnHeapNotInTransaction;
	}

	AutoRTFM::Testing::Commit([&]
	{
		int OnStackInTransaction = 3;
		REQUIRE(AutoRTFM::IsOnCurrentTransactionStack(&OnStackInTransaction));

		int* OnHeapInTransaction = new int{4};
		REQUIRE(!AutoRTFM::IsOnCurrentTransactionStack(OnHeapInTransaction));
		delete OnHeapInTransaction;

		AutoRTFM::Testing::Commit([&]
		{
			// `OnStackInTransaction` is no longer in the innermost scope.
			REQUIRE(!AutoRTFM::IsOnCurrentTransactionStack(&OnStackInTransaction));

			int OnInnermostStackInTransaction = 5;
			REQUIRE(AutoRTFM::IsOnCurrentTransactionStack(&OnInnermostStackInTransaction));
		});
	});
}

TEST_CASE("API.OnCompleteIgnoredOutsideTransaction")
{
	SECTION("Callback not called outside transaction")
	{
		AutoRTFM::OnComplete([&] { FAIL("Unreachable"); });
	}
}

TEST_CASE("API.CascadingAbortTransaction_WithinUnscopedTransaction")
{
	AutoRTFM::Transact([]
	{
		AutoRTFM::TransactThenOpen([]
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::CascadingAbortTransaction();
			REQUIRE(AutoRTFM::ContextStatus() == AutoRTFM::EContextStatus::Unwinding);
		});

		FAIL("unreachable");
	});
	REQUIRE(AutoRTFM::ContextStatus() == AutoRTFM::EContextStatus::Idle);
}

TEST_CASE("API.CascadingRetryTransaction")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

	size_t CallCount = 0;
	bool bTransactionalWrite = false;

	SECTION("Non-nested committed transaction")
	{
		AutoRTFM::Testing::Commit([&]
		{
			bTransactionalWrite = true;
			if (CallCount == 0)
			{
				AutoRTFM::OnRetry([&]
				{
					// The write above would have been undone.
					REQUIRE(!bTransactionalWrite);
					CallCount++;
				});
				AutoRTFM::CascadingRetryTransaction();
			}
		});
		REQUIRE(bTransactionalWrite);
		REQUIRE(CallCount == 1);
	}

	SECTION("Non-nested aborted transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			bTransactionalWrite = true;
			if (CallCount == 0)
			{
				AutoRTFM::OnRetry([&]
				{
					// The write above would have been undone.
					REQUIRE(!bTransactionalWrite);
					CallCount++;
				});
				AutoRTFM::CascadingRetryTransaction();
			}

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bTransactionalWrite);
		REQUIRE(CallCount == 1);
	}

	SECTION("Nested committed transaction")
	{
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Commit([&]
			{
				bTransactionalWrite = true;
				if (CallCount == 0)
				{
					AutoRTFM::OnRetry([&]
					{
						// The write above would have been undone.
						REQUIRE(!bTransactionalWrite);
						CallCount++;
					});
					AutoRTFM::CascadingRetryTransaction();
				}
			});
		});
		REQUIRE(bTransactionalWrite);
		REQUIRE(CallCount == 1);
	}

	SECTION("Nested aborted transaction")
	{
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Abort([&]
			{
				bTransactionalWrite = true;
				if (CallCount == 0)
				{
					AutoRTFM::OnRetry([&]
					{
						// The write above would have been undone.
						REQUIRE(!bTransactionalWrite);
						CallCount++;
					});
					AutoRTFM::CascadingRetryTransaction();
				}

				AutoRTFM::AbortTransaction();
			});
		});
		REQUIRE(!bTransactionalWrite);
		REQUIRE(CallCount == 1);
	}

	SECTION("Open")
	{
		AutoRTFM::Testing::Commit([&]
		{
			bTransactionalWrite = true;
			if (CallCount == 0)
			{
				AutoRTFM::OnRetry([&]
				{
					// The write above would have been undone.
					REQUIRE(!bTransactionalWrite);
					CallCount++;
				});
				AutoRTFM::Open([&]
				{
					AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
					{
						AutoRTFM::Open([&]
						{
							AutoRTFM::CascadingRetryTransaction();
							REQUIRE(AutoRTFM::ContextStatus() == AutoRTFM::EContextStatus::Unwinding);
						});
					});
					REQUIRE(AutoRTFM::ContextStatus() == AutoRTFM::EContextStatus::Unwinding);
					REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingRetry);
				});
			}
		});
		REQUIRE(bTransactionalWrite);
		REQUIRE(CallCount == 1);
	}

	SECTION("Status query during retry")
	{
		AutoRTFM::Testing::Commit([&]
		{
			if (CallCount == 0)
			{
				AutoRTFM::OnRetry([&]
				{
					REQUIRE(!AutoRTFM::IsTransactional());
					REQUIRE(!AutoRTFM::IsClosed());
					REQUIRE(!AutoRTFM::IsCommitting());
					REQUIRE(AutoRTFM::IsRetrying());
					REQUIRE(AutoRTFM::IsCommittingOrAborting());
					CallCount++;
				});
				AutoRTFM::CascadingRetryTransaction();
			}
		});
		REQUIRE(CallCount == 1);
	}
}

TEST_CASE("API.IsAutoRTFMRuntimeEnabled")
{
	AutoRTFM::Testing::FEnabledStateResetterScoped _(AutoRTFM::ForTheRuntime::AutoRTFM_EnabledByDefault);

	// On entry to the test we'll be enabled-by-default, so first test we can set back to disabled by default.
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_DisabledByDefault));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());

	// Now move up a priority level to enabled, and check that we cannot set back to enabled-by-default.
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Enabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_DisabledByDefault));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Disabled));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_EnabledByDefault));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());

	// Now move up a priority level to overridden-enabled, and check we cannot set back to enabled or enabled-by-default.
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_OverriddenEnabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_DisabledByDefault));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Disabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_OverriddenDisabled));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_EnabledByDefault));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Enabled));
	REQUIRE(!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());

	// And lastly set force-enabled, and check nothing else can change.
	REQUIRE(AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedEnabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedDisabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_OverriddenEnabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_OverriddenDisabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Enabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Disabled));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_EnabledByDefault));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
	REQUIRE(!AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_DisabledByDefault));
	REQUIRE(AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled());
}

TEST_CASE("API.CoinTossDisable")
{
	SECTION("With default enablement")
	{
		AutoRTFM::Testing::FEnabledStateResetterScoped _(AutoRTFM::ForTheRuntime::AutoRTFM_EnabledByDefault);

		// Set the chance of disabling to 100.0, effectively disabling the coin toss.
		AutoRTFM::ForTheRuntime::SetAutoRTFMEnabledProbability(100.0f);
		REQUIRE(!AutoRTFM::ForTheRuntime::CoinTossDisable());

		// Set the chance of disabling to 0.0, always disabling by coin toss.
		AutoRTFM::ForTheRuntime::SetAutoRTFMEnabledProbability(0.0f);
		REQUIRE(AutoRTFM::ForTheRuntime::CoinTossDisable());
	}
	
	SECTION("With force enablement")
	{
		AutoRTFM::Testing::FEnabledStateResetterScoped _(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedEnabled);

		// Set the chance of enabling to 0.0, always disabling by coin toss - but this gets ignored because
		// we are set to force enable.
		AutoRTFM::ForTheRuntime::SetAutoRTFMEnabledProbability(0.0f);
		REQUIRE(!AutoRTFM::ForTheRuntime::CoinTossDisable());
	}

	SECTION("Already disabled")
	{
		AutoRTFM::Testing::FEnabledStateResetterScoped _(AutoRTFM::ForTheRuntime::AutoRTFM_DisabledByDefault);

		// Set the chance of disabling to 0.0, always disabling by coin toss.
		AutoRTFM::ForTheRuntime::SetAutoRTFMEnabledProbability(0.0f);
		REQUIRE(!AutoRTFM::ForTheRuntime::CoinTossDisable());
	}
}
