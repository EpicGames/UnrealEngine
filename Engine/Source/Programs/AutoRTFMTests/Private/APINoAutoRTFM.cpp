// Copyright Epic Games, Inc. All Rights Reserved.

#include "API.h"
#include "AutoRTFM.h"

UE_AUTORTFM_NOAUTORTFM int NoAutoRTFM::DoSomethingC(int I)
{
    return I + 13;
}

UE_AUTORTFM_NOAUTORTFM int NoAutoRTFM::DoSomethingInTransactionC(int I)
{
    return I + 42;
}

UE_AUTORTFM_NOAUTORTFM int NoAutoRTFM::DoSomethingCpp(int I)
{
    return I + 13;
}

UE_AUTORTFM_NOAUTORTFM int NoAutoRTFM::DoSomethingInTransactionCpp(int I)
{
    return I + 42;
}
