// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "AutoRTFM/AutoRTFMUE.h"
#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Async/SharedMutex.h"
#include "Catch2Includes.h"
#include "Containers/MpscQueue.h"
#include "Containers/SpscQueue.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Logging/LogScopedVerbosityOverride.h"

#include <map>
#include <vector>

TEST_CASE("Abort")
{
    int x = 42;
    std::vector<int> v;
    std::map<int, std::vector<int>> m;
    v.push_back(100);
    m[1].push_back(2);
    m[1].push_back(3);
    m[4].push_back(5);
    m[6].push_back(7);
    m[6].push_back(8);
    m[6].push_back(9);

	auto transaction = AutoRTFM::Transact([&]()
    {
		x = 5;
    	for (size_t n = 10; n--;)
    		v.push_back(2 * n);
    	m.clear();
    	m[10].push_back(11);
    	m[12].push_back(13);
    	m[12].push_back(14);
    	AutoRTFM::AbortTransaction();
	});

    REQUIRE(
        AutoRTFM::ETransactionResult::AbortedByRequest ==
		transaction);
    REQUIRE(x == 42);
    REQUIRE(v.size() == 1);
    REQUIRE(v[0] == 100);
    REQUIRE(m.size() == 3);
    REQUIRE(m[1].size() == 2);
    REQUIRE(m[1][0] == 2);
    REQUIRE(m[1][1] == 3);
    REQUIRE(m[4].size() == 1);
    REQUIRE(m[4][0] == 5);
    REQUIRE(m[6].size() == 3);
    REQUIRE(m[6][0] == 7);
    REQUIRE(m[6][1] == 8);
    REQUIRE(m[6][2] == 9);
}

TEST_CASE("Abort.NestedAbortOrder")
{
	unsigned Orderer = 0;

	AutoRTFM::Testing::Commit([&]
	{
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnAbort([&]
		{
			Orderer = 0;
		});

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			AutoRTFM::OnAbort([&]
				{
					REQUIRE(3 == Orderer);
					Orderer += 1;
				});

			AutoRTFM::OnPreAbort([&]
				{
					REQUIRE(1 == Orderer);
					Orderer += 1;
				});

			AutoRTFM::OnAbort([&]
				{
					REQUIRE(2 == Orderer);
					Orderer += 1;
				});

			AutoRTFM::OnPreAbort([&]
				{
					REQUIRE(0 == Orderer);
					Orderer += 1;
				});

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	});

	REQUIRE(4 == Orderer);
}

TEST_CASE("Abort.TransactionInOnCommit")
{
	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::OnCommit([&]
		{
			bool bDidSomething = false;

			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(AutoRTFM::ETransactionResult::RejectedTransactDuringCommit == Result);
			REQUIRE(false == bDidSomething);
		});
	});

}

TEST_CASE("Abort.TransactionInOnAbort")
{
	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		AutoRTFM::OnAbort([&]
		{
			bool bDidSomething = false;

			AutoRTFM::ETransactionResult InnerResult = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(AutoRTFM::ETransactionResult::RejectedTransactDuringAbort == InnerResult);
			REQUIRE(false == bDidSomething);
		});

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
}

TEST_CASE("Abort.TransactionInOnPreAbort")
{
	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		AutoRTFM::OnPreAbort([&]
		{
			bool bDidSomething = false;

			AutoRTFM::ETransactionResult InnerResult = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(AutoRTFM::ETransactionResult::RejectedTransactDuringAbort == InnerResult);
			REQUIRE(false == bDidSomething);
		});

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
}

TEST_CASE("Abort.TransactionInAbortedOnComplete")
{
	AutoRTFM::Testing::Abort([&]
	{
		AutoRTFM::OnComplete([&]
		{
			bool bDidSomething = false;

			AutoRTFM::ETransactionResult InnerResult = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(InnerResult == AutoRTFM::ETransactionResult::RejectedTransactDuringCompletion);
			REQUIRE(!bDidSomething);
		});

		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("Abort.TransactionInOnComplete")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

	size_t CallCount = 0;
	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::OnComplete([&]
		{
			CallCount++;

			bool bDidSomething = false;

			AutoRTFM::ETransactionResult InnerResult = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(!bDidSomething);
			REQUIRE(InnerResult == AutoRTFM::ETransactionResult::RejectedTransactDuringCompletion);
		});
	});
	REQUIRE(CallCount == 1);
}

TEST_CASE("Abort.TransactionInOnRetry")
{
	AUTORTFM_SCOPED_DISABLE_RETRY();

	size_t CallCount = 0;
	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::OnRetry([&]
		{
			CallCount++;

			bool bDidSomething = false;

			AutoRTFM::ETransactionResult InnerResult = AutoRTFM::Transact([&]
			{
				bDidSomething = true;
			});

			REQUIRE(!bDidSomething);
			REQUIRE(InnerResult == AutoRTFM::ETransactionResult::RejectedTransactDuringRetry);
		});
		
		if (CallCount == 0)
		{
			AutoRTFM::CascadingRetryTransaction();
		}
	});

	REQUIRE(CallCount == 1);
}

TEST_CASE("Abort.AbortInInnerTransaction")
{
	int Value = 1;

	AutoRTFM::Transact([&]
	{
		Value = 2;
		AutoRTFM::Transact([&]
		{
			Value = 3;
			AutoRTFM::AbortTransaction();  // Only cancels innermost nest.
		});
	});

	REQUIRE(Value == 2);
}

TEST_CASE("Abort.OnAbortInOuterTransaction_AbortInInnerTransaction")
{
	int Value = 1;

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnAbort([&]
		{
			if (AutoRTFM::ForTheRuntime::GetRetryTransaction() == AutoRTFM::ForTheRuntime::NoRetry)
			{
				REQUIRE(!"This should not be reached.");
				Value = 2;
			}
		});

		AutoRTFM::Transact([&]
		{
			REQUIRE(Value == 1);
			Value = 3;
			AutoRTFM::AbortTransaction();  // Only cancels innermost nest.
		});

		REQUIRE(Value == 1);
		Value = 4;
	});

	REQUIRE(Value == 4);
}

TEST_CASE("Abort.OnPreAbortInOuterTransaction_AbortInInnerTransaction")
{
	int Value = 1;

	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnPreAbort([&]
		{
			if (AutoRTFM::ForTheRuntime::GetRetryTransaction() == AutoRTFM::ForTheRuntime::NoRetry)
			{
				REQUIRE(!"This should not be reached.");
				Value = 2;
			}
		});

		AutoRTFM::Transact([&]
		{
			REQUIRE(Value == 1);
			Value = 3;
			AutoRTFM::AbortTransaction();  // Only cancels innermost nest.
		});

		REQUIRE(Value == 1);
		Value = 4;
	});

	REQUIRE(Value == 4);
}

TEST_CASE("Abort.Cascade")
{
	bool bTouched = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;
		AutoRTFM::Transact([&]
		{
			AutoRTFM::CascadingAbortTransaction();
		});

		FAIL("Execution should never reach this point");
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(false == bTouched);
}

TEST_CASE("Abort.CascadeWithOnCompleteCallback")
{
	bool bTouched = false;
	bool bPostAbortCallbackWasExecuted = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;
		AutoRTFM::Transact([&]
		{
			AutoRTFM::OnComplete([&]
			{
				REQUIRE(!bTouched);  // Rollbacks must occur before the post-abort callback is invoked.
				bPostAbortCallbackWasExecuted = true;
			});
			AutoRTFM::CascadingAbortTransaction();
		});

		FAIL("Execution should never reach this point");
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(!bTouched);
	REQUIRE(bPostAbortCallbackWasExecuted);
}

TEST_CASE("Abort.CascadingAbortMustRunOnAbortsBeforeOnComplete")
{
	bool bOnPreAbortWasCalled = false;
	bool bOnAbortWasCalled = false;
	bool bPostAbortCallbackWasExecuted = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		AutoRTFM::OnPreAbort([&]
		{
			REQUIRE(!bOnAbortWasCalled);  // OnPreAborts must be before OnAborts.
			bOnPreAbortWasCalled = true;
		});
		AutoRTFM::OnAbort([&]
		{
			REQUIRE(bOnPreAbortWasCalled);  // OnAborts must be after OnPreAborts.
			bOnAbortWasCalled = true;
		});
		AutoRTFM::OnComplete([&]
		{
			REQUIRE(bOnAbortWasCalled);  // OnAborts must before the post-abort callback is invoked.
			bPostAbortCallbackWasExecuted = true;
		});
		AutoRTFM::CascadingAbortTransaction();
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(bOnAbortWasCalled);
	REQUIRE(bOnPreAbortWasCalled);
	REQUIRE(bPostAbortCallbackWasExecuted);
}

TEST_CASE("Abort.CascadingAbortRollbackTransaction")
{
	bool bTouched = false;
	bool bExecutedCodeAfterCascadingRollback = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::CascadingAbortTransaction();
			bExecutedCodeAfterCascadingRollback = true;
		});
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(!bTouched);
	REQUIRE(bExecutedCodeAfterCascadingRollback);
}

TEST_CASE("Abort.CascadeThroughOpen")
{
	bool bTouched = false;
	bool bOnAbortCallbackWasExecuted = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;

		AutoRTFM::OnAbort([&]
		{
			REQUIRE(!bTouched);  // Rollbacks must occur before the post-abort callback is invoked.
			bOnAbortCallbackWasExecuted = true;
		});
		
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::Transact([&]
				{
					AutoRTFM::CascadingAbortTransaction();
				});

				FAIL("AutoRTFM::Transact() failed to throw after a cascading abort");
			});

			REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingAbort);
		});
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(bOnAbortCallbackWasExecuted);
	REQUIRE(!bTouched);
}

TEST_CASE("Abort.CascadeWithOnCompleteThroughOpen")
{
	bool bTouched = false;
	bool bOnCompleteCallbackWasExecuted = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::Transact([&]
				{
					AutoRTFM::OnComplete([&]
					{
						REQUIRE(!bTouched);  // Rollbacks must occur before the complete callback is invoked.
						bOnCompleteCallbackWasExecuted = true;
					});
					AutoRTFM::CascadingAbortTransaction();
				});

				FAIL("AutoRTFM::Transact() failed to throw after a cascading abort");
			});

			REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingAbort);
		});
		FAIL("AutoRTFM::Open() failed to throw after a cascading abort");
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(!bTouched);
	REQUIRE(bOnCompleteCallbackWasExecuted);
}

TEST_CASE("Abort.CascadeWithOnCompleteThroughFunctionAttribute")
{
	struct FState
	{
		AUTORTFM_OPEN void AutoRTFMOpen()
		{
			CallCloseTransactCascadingAbort();
		}

		AUTORTFM_OPEN_NO_VALIDATION void AutoRTFMOpenNoValidation()
		{
			CallCloseTransactCascadingAbort();
		}

		UE_AUTORTFM_ALWAYS_OPEN void UEAutoRTFMAlwaysOpen()
		{
			CallCloseTransactCascadingAbort();
		}

		UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION void UEAutoRTFMAlwaysOpenNoMemoryValidation()
		{
			CallCloseTransactCascadingAbort();
		}

		void CallCloseTransactCascadingAbort()
		{
			const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::Transact([&]
				{
					AutoRTFM::OnComplete([&]
					{
						REQUIRE(!bTouched);  // Rollbacks must occur before the post-abort callback is invoked.
						bOnCompleteCallbackWasExecuted = true;
					});
					AutoRTFM::CascadingAbortTransaction();
				});
			});
			REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingAbort);
		}

		bool bTouched = false;
		bool bOnCompleteCallbackWasExecuted = false;
	};
	FState State;

	AutoRTFM::ETransactionResult Result = AutoRTFM::ETransactionResult::Committed;
	SECTION("AUTORTFM_OPEN")
	{
		Result = AutoRTFM::Transact([&]
		{
			State.bTouched = true;
			State.AutoRTFMOpen();
			FAIL("AUTORTFM_OPEN function failed to throw after an abort");
		});
	}
	SECTION("AUTORTFM_OPEN_NO_VALIDATION")
	{
		Result = AutoRTFM::Transact([&]
		{
			State.bTouched = true;
			State.AutoRTFMOpenNoValidation();
			FAIL("AUTORTFM_OPEN_NO_VALIDATION function failed to throw after an abort");
		});
	}
	SECTION("UE_AUTORTFM_ALWAYS_OPEN")
	{
		Result = AutoRTFM::Transact([&]
		{
			State.bTouched = true;
			State.UEAutoRTFMAlwaysOpen();
			FAIL("UE_AUTORTFM_ALWAYS_OPEN function failed to throw after an abort");
		});
	}
	SECTION("UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION")
	{
		Result = AutoRTFM::Transact([&]
		{
			State.bTouched = true;
			State.UEAutoRTFMAlwaysOpenNoMemoryValidation();
			FAIL("UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION function failed to throw after an abort");
		});
	}
	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(!State.bTouched);
	REQUIRE(State.bOnCompleteCallbackWasExecuted);
}

TEST_CASE("Abort.CascadeThroughManualTransaction")
{
	bool bOnCompleteCallbackWasExecuted = false;
	bool bTouched = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::OnComplete([&]
				{
					REQUIRE(!bTouched);  // Rollbacks must occur before the post-abort callback is invoked.
					bOnCompleteCallbackWasExecuted = true;
				});
				AutoRTFM::CascadingAbortTransaction();
			});

			REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingAbort);
		});
	});

	REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByCascadingAbort);
	REQUIRE(!bTouched);
	REQUIRE(bOnCompleteCallbackWasExecuted);
}

static int SAbortKey;

TEST_CASE("Abort.PushOnAbortHandler_NoAbort")
{
	int Value = 55;

	AutoRTFM::Testing::Commit([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value = 77; });
	});

	REQUIRE(Value == 66);
}

TEST_CASE("Abort.PushOnAbortHandler_WithAbort")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value = 77; });

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	REQUIRE(Value == 77);
}

TEST_CASE("Abort.PushOnAbortHandler_MutableCapture")
{
	FString Message = "Hello";

	AutoRTFM::Transact([&]
	{
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [MessageCopy = Message]() mutable
		{
			MessageCopy += " World!";
			REQUIRE(MessageCopy == "Hello World!");
		});

		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("Abort.PushOnAbortHandler_WithPop_NoAbort")
{
	int Value = 55;

	AutoRTFM::Testing::Commit([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 77; });
		Value = 88;

		AutoRTFM::PopOnAbortHandler(&SAbortKey);
	});

	REQUIRE(Value == 88);
}

TEST_CASE("Abort.PushOnAbortHandler_WithPopAll_NoAbort")
{
	int Value = 55;

	AutoRTFM::Testing::Commit([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value = 77; });
		Value = 88;

		AutoRTFM::PopAllOnAbortHandlers(&SAbortKey);
	});

	REQUIRE(Value == 88);
}

TEST_CASE("Abort.PushOnAbortHandler_WithPop_WithAbort")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value = 77; });
		Value = 88;

		AutoRTFM::PopOnAbortHandler(&SAbortKey);

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	REQUIRE(Value == 55);
}

TEST_CASE("Abort.PushOnAbortHandler_WithPopAll_WithAbort")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 77; });
		Value = 88;

		AutoRTFM::PopAllOnAbortHandlers(&SAbortKey);

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	REQUIRE(Value == 55);
}

TEST_CASE("Abort.PushOnAbortHandler_Duplicates1")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 77; });
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 88; });
		Value = 99;

		AutoRTFM::PopOnAbortHandler(&SAbortKey);

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

	// The first push on abort will still go through.
	REQUIRE(Value == 77);
}

TEST_CASE("Abort.PushOnAbortHandler_PopAll_Duplicates")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 77; });
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { Value = 88; });
		Value = 99;

		AutoRTFM::PopAllOnAbortHandlers(&SAbortKey);

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);

	// No abort handlers should execute.
	REQUIRE(Value == 55);
}

TEST_CASE("Abort.PushOnAbortHandler_Duplicates2")
{
	int Value = 55;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		Value = 66;
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value += 12; });
		AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value](){ Value = 65; });
		Value = 99;

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	REQUIRE(Value == 77);
}

TEST_CASE("Abort.PushOnAbortHandler_Order")
{
	SECTION("HandlerSandwich")
	{
		SECTION("WithoutPop")
		{
			int Value = 37;

			const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([&Value] { REQUIRE(42 == Value); Value += 1; });
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { REQUIRE(40 == Value); Value += 2; });
				AutoRTFM::OnAbort([&Value] { REQUIRE(37 == Value); Value += 3; });

				Value = 99;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(Value == 43);
		}

		SECTION("WithPop")
		{
			int Value = 37;

			const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([&Value] { REQUIRE(40 == Value); Value += 1; });
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { REQUIRE(false); });
				AutoRTFM::OnAbort([&Value] { REQUIRE(37 == Value); Value += 3; });

				AutoRTFM::PopOnAbortHandler(&SAbortKey);

				Value = 99;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(Value == 41);
		}
	}

	SECTION("HandlerInChild")
	{
		SECTION("WithoutPop")
		{
			int Value = 37;

			const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([&Value]
				{
					REQUIRE(42 == Value);
					Value += 1;
				});

				// Make a child transaction.
				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]()
					{
						// If we are retrying nested transactions too, we can't check that
						// the value was something specific before hand!
						if (!AutoRTFM::ForTheRuntime::ShouldRetryNestedTransactionsToo())
						{
							REQUIRE(40 == Value);
							Value += 2;
						}
						else
						{
							Value += 1;
						}
					});
				});

				AutoRTFM::OnAbort([&Value]
				{
					// If we are retrying nested transactions too, we've ran the on-abort in the
					// child transaction once, so our value will be larger.
					if (!AutoRTFM::ForTheRuntime::ShouldRetryNestedTransactionsToo())
					{
						REQUIRE(37 == Value);
					}
					else
					{
						REQUIRE(38 == Value);
					}

					Value += 3;
				});

				Value = 99;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(Value == 43);
		}

		SECTION("WithPop")
		{
			int Value = 37;

			const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([&Value] { REQUIRE(40 == Value); Value += 1; });

				// Make a child transaction.
				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]()
					{
						// Only if we are retrying on
						REQUIRE(AutoRTFM::ForTheRuntime::ShouldRetryNestedTransactionsToo());
					});
				});

				AutoRTFM::OnAbort([&Value] { REQUIRE(37 == Value); Value += 3; });

				// Bit funky, but we can pop the child's push here!
				AutoRTFM::PopOnAbortHandler(&SAbortKey);

				Value = 99;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(Value == 41);
		}

		SECTION("AbortInChild")
		{
			int Value = 99;

			AutoRTFM::ETransactionResult Result = AutoRTFM::ETransactionResult::Committed;
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::OnCommit([&Value] { REQUIRE(37 == Value); Value += 1; });

				// Make a child transaction.
				Result = AutoRTFM::Transact([&]
				{
					AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value]() { REQUIRE(99 == Value); Value += 2; });
					AutoRTFM::AbortTransaction();
				});

				AutoRTFM::Open([&] AUTORTFM_DISABLE
				{
					REQUIRE(Value == 101);
				});

				AutoRTFM::OnCommit([&Value] { REQUIRE(38 == Value); Value += 3; });

				Value = 37;

				AutoRTFM::OnAbort([&Value] { Value = 99; });
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(Value == 41);
		}

		SECTION("PopInChild")
		{
			int Value = 99;
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 42; });
				AutoRTFM::Testing::Commit([&] { AutoRTFM::PopOnAbortHandler(&SAbortKey); });
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(99 == Value);
		}

		SECTION("PopInChildAndAbort")
		{
			int Value = 99;
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 42; });

				AutoRTFM::Transact([&]
				{
					AutoRTFM::PopOnAbortHandler(&SAbortKey);

					// This abort means the pop on abort handler *should not* propagate
					// to the parent scope (meaning the parents push on abort should
					// run as normal).
					AutoRTFM::AbortTransaction();
				});

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(42 == Value);
		}

		SECTION("PopInChildsChild")
		{
			int Value = 99;
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 42; });
				AutoRTFM::Testing::Commit([&] { AutoRTFM::Testing::Commit([&] { AutoRTFM::PopOnAbortHandler(&SAbortKey); }); });
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(99 == Value);
		}

		SECTION("PopAllInChild")
		{
			int Value = 99;
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 42; });
				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 13; });
					AutoRTFM::PopAllOnAbortHandlers(&SAbortKey);
				});
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(99 == Value);
		}

		SECTION("PopAllInChildAbort")
		{
			int Value = 99;
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 42; });
				AutoRTFM::Transact([&]
				{
					AutoRTFM::PushOnAbortHandler(&SAbortKey, [&Value] { Value = 13; });
					AutoRTFM::PopAllOnAbortHandlers(&SAbortKey);

					// This abort will ensure that the pop-all cannot affect the outer transactions
					// push on abort!
					AutoRTFM::AbortTransaction();
				});
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(42 == Value);
		}
	}
}

TEST_CASE("Abort.OnAbortTiming")
{
	bool bOnPreAbortRan = false;
	bool bOnAbortRan = false;
	int Memory = 666;
	AutoRTFM::Testing::Commit([&]
	{
		// If we are retrying transactions, need to reset the test state.
		AutoRTFM::OnPreAbort([&]
		{
			REQUIRE(bOnPreAbortRan);
			REQUIRE(Memory == 666);
			bOnPreAbortRan = false;
		});
		AutoRTFM::OnAbort([&]
		{
			REQUIRE(bOnAbortRan);
			REQUIRE(Memory == 666);
			bOnAbortRan = false;
		});

		REQUIRE(bOnAbortRan == false);
		REQUIRE(Memory == 666);

		AutoRTFM::Transact([&]
		{
			Memory = 1234;
			REQUIRE(Memory == 1234);

			AutoRTFM::OnPreAbort([&]
			{
				REQUIRE(Memory == 1234);
				REQUIRE(bOnAbortRan == false);
				bOnPreAbortRan = true;
			});
			AutoRTFM::OnAbort([&]
			{
				REQUIRE(Memory == 666);
				REQUIRE(bOnPreAbortRan == true);
				bOnAbortRan = true;
			});

			AutoRTFM::AbortTransaction();
		});
	});
	REQUIRE(Memory == 666);
	REQUIRE(bOnAbortRan == true);
	REQUIRE(bOnPreAbortRan == true);
}

static void FnHasNoClosed()
{
	(void)fopen("fopen() is not supported in a closed transaction", "rb");
}

TEST_CASE("Abort.Language")
{
	AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
	AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2(false);

	bool bTouched = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;
		FnHasNoClosed();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
	REQUIRE(false == bTouched);
}

TEST_CASE("Abort.LanguageThroughOpen")
{
	AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
	AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2(false);

	bool bTouched = false;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		bTouched = true;

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				FnHasNoClosed();
			});

			REQUIRE(Status == AutoRTFM::ETransactionStatus::AbortedByLanguage);
		});
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
	REQUIRE(false == bTouched);
}

// Test for SOL-5804
TEST_CASE("Abort.StackWriteToOuterOpen")
{
	bool WritesUndone = true;
	bool Success = false;

	AutoRTFM::Testing::Abort([&]
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			std::array<int, 64> Values{};

			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				// On stack outside transaction.
				// Should be reverted as part of the abort.
				WritesUndone = false;

				// On stack inside transaction.
				// Writes should not be reverted as part of the abort.
				for (size_t I = 0; I < Values.size(); I++)
				{
					Values[I] = static_cast<int>(I * 10);
				}
			});

			REQUIRE(Status == AutoRTFM::ETransactionStatus::Executing);
		});

		// If any of the variables on the stack within the Open() get written to
		// on abort, then it should change the values of this array.
		std::array<int, 64> StackGuard{};

		// The OnPreAbort handler should be called *before* the memory is reverted.
		AutoRTFM::OnPreAbort([&]
		{
			if (WritesUndone)
			{
				FAIL("OnPreAbort was called after reverting memory");
			}
			else if (StackGuard != std::array<int, 64>{})
			{
				FAIL("StackGuard was corrupted");
			}
		});
		// The OnAbort handler should be called *after* the memory is reverted.
		AutoRTFM::OnAbort([&]
		{
			if (!WritesUndone)
			{
				FAIL("OnAbort was called without first reverting memory");
			}
			else if (StackGuard != std::array<int, 64>{})
			{
				FAIL("StackGuard was corrupted");
			}
			else
			{
				Success = true;
			}
		});

		// Do the abort!
		AutoRTFM::AbortTransaction();
	});

	REQUIRE(Success);
}

TEST_CASE("Abort.PushOnAbortOrdering")
{
	SECTION("InParent")
	{
		unsigned Orderer = 0;
		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&] { REQUIRE(6 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(5 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(4 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(false); });
			AutoRTFM::OnAbort([&] { REQUIRE(3 == Orderer++); });
			AutoRTFM::PopOnAbortHandler(&Orderer);
			AutoRTFM::OnAbort([&] { REQUIRE(2 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(1 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(0 == Orderer++); });
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("InChild")
	{
		// The inner transaction cannot fail in this example so lets skip the test if we are testing that.
		if (AutoRTFM::ForTheRuntime::ShouldRetryNestedTransactionsToo())
		{
			return;
		}

		unsigned Orderer = 0;
		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&] { REQUIRE(20 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(19 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(18 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(false); });
			AutoRTFM::OnAbort([&] { REQUIRE(17 == Orderer++); });
			AutoRTFM::PopOnAbortHandler(&Orderer);
			AutoRTFM::OnAbort([&] { REQUIRE(16 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(15 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(14 == Orderer++); });

			// This commits which will add the on abort handlers to the parent scope.
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::OnAbort([&] { REQUIRE(13 == Orderer++); });
				AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(12 == Orderer++); });
				AutoRTFM::OnAbort([&] { REQUIRE(11 == Orderer++); });
				AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(false); });
				AutoRTFM::OnAbort([&] { REQUIRE(10 == Orderer++); });
				AutoRTFM::PopOnAbortHandler(&Orderer);
				AutoRTFM::OnAbort([&] { REQUIRE(9 == Orderer++); });
				AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(8 == Orderer++); });
				AutoRTFM::OnAbort([&] { REQUIRE(7 == Orderer++); });
			});

			AutoRTFM::OnAbort([&] { REQUIRE(6 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(5 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(4 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(false); });
			AutoRTFM::OnAbort([&] { REQUIRE(3 == Orderer++); });
			AutoRTFM::PopOnAbortHandler(&Orderer);
			AutoRTFM::OnAbort([&] { REQUIRE(2 == Orderer++); });
			AutoRTFM::PushOnAbortHandler(&Orderer, [&] { REQUIRE(1 == Orderer++); });
			AutoRTFM::OnAbort([&] { REQUIRE(0 == Orderer++); });

			AutoRTFM::AbortTransaction();
		});
	}
}

#if AUTORTFM_SANITIZER
TEST_CASE("Abort.CallbackModifiesTransactionalMemory")
{
	AUTORTFM_SCOPED_SANITIZER_MODE_WARN();
	AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	int Memory = 0;

	SECTION("OnAbort")
	{
		AutoRTFM::Testing::Commit([&]
		{
			Memory = 10;
			AutoRTFM::Testing::Abort([&]
			{
				AutoRTFM::OnAbort([&]
				{
					Memory = 20;
				});
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
	}

	SECTION("OnPreAbort")
	{
		AutoRTFM::Testing::Commit([&]
		{
			Memory = 10;
			AutoRTFM::Testing::Abort([&]
			{
				AutoRTFM::OnPreAbort([&]
				{
					Memory = 20;
				});
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
	}

	SECTION("Destructor")
	{
		struct AUTORTFM_ENABLE IncrementOnDtor
		{
			~IncrementOnDtor() { (*Ptr)++; }
			int* const Ptr;
		};

		SECTION("OnAbort")
		{
			AutoRTFM::Testing::Commit([&]
			{
				Memory = 10;
				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnAbort([Dtor = IncrementOnDtor{&Memory}]
					{
					});
					AutoRTFM::AbortTransaction();
				});
			});

			REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
		}

		SECTION("OnPreAbort")
		{
			AutoRTFM::Testing::Commit([&]
			{
				Memory = 10;
				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnPreAbort([Dtor = IncrementOnDtor{&Memory}]
					{
					});
					AutoRTFM::AbortTransaction();
				});
			});

			REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
		}

		SECTION("OnCommit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				Memory = 10;
				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnCommit([Dtor = IncrementOnDtor{&Memory}]
					{
					});
					AutoRTFM::AbortTransaction();
				});
			});

			REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
		}
	}
}
#endif // AUTORTFM_SANITIZER

namespace
{

class FScopedErrorHintTest
{
	private:
		AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1{AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort};
		AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2{false};
};

}

TEST_CASE("Abort.Language.ErrorHints.CriticalSection")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;
	AutoRTFM::ETransactionResult Result;

	SECTION("InitializeCriticalSection")
	{
		Result = AutoRTFM::Transact([&]
		{
			FCriticalSection CS;
		});
	};

	SECTION("EnterCriticalSection")
	{
		FCriticalSection CS;

		Result = AutoRTFM::Transact([&]
		{
			CS.Lock();
		});
	};

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our detailed error message?
	REQUIRE(WarningContext.HasWarningSubstring("A critical section was accessed while inside an AutoRTFM transaction."));
	REQUIRE(WarningContext.HasWarningSubstring("AutoRTFM/Documentation/README.md"));
	REQUIRE(WarningContext.HasWarningSubstring("Use the call stack below to find the FCriticalSection"));
}

TEST_CASE("Abort.Language.ErrorHints.RWLock")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	FRWLock RW;
	AutoRTFM::ETransactionResult Result;

	SECTION("WriteLock")
	{
		Result = AutoRTFM::Transact([&]
		{
			RW.WriteLock();
		});
	}

	SECTION("TryReadLock")
	{
		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(RW.TryReadLock());
		});
	}

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	REQUIRE(WarningContext.HasWarningSubstring("A read-write lock was accessed while inside an AutoRTFM transaction."));
}

TEST_CASE("Abort.Language.ErrorHints.Mutex")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	UE::FMutex Mutex;
	AutoRTFM::ETransactionResult Result;

	SECTION("Lock")
	{
		Result = AutoRTFM::Transact([&]
		{
			Mutex.Lock();
		});
	}

	SECTION("IsLocked?")
	{
		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(!Mutex.IsLocked());
		});
	}

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	REQUIRE(WarningContext.HasWarningSubstring("A mutex was accessed while inside an AutoRTFM transaction."));
}

TEST_CASE("Abort.Language.ErrorHints.SharedMutex")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	UE::FSharedMutex Mutex;
	AutoRTFM::ETransactionResult Result;

	SECTION("LockShared")
	{
		Result = AutoRTFM::Transact([&]
		{
			Mutex.LockShared();
		});
	}

	SECTION("IsLocked?")
	{
		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(!Mutex.IsLockShared());
		});
	}

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	REQUIRE(WarningContext.HasWarningSubstring("A shared mutex was accessed while inside an AutoRTFM transaction."));
}

TEST_CASE("Abort.Language.ErrorHints.RecursiveMutex")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	UE::FRecursiveMutex Mutex;
	AutoRTFM::ETransactionResult Result;

	SECTION("Lock")
	{
		Result = AutoRTFM::Transact([&]
		{
			Mutex.Lock();
		});
	}

	SECTION("IsLocked?")
	{
		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(!Mutex.IsLocked());
		});
	}

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	REQUIRE(WarningContext.HasWarningSubstring("A recursive mutex was accessed while inside an AutoRTFM transaction."));
}

#if 0
// Testing an open free is flaky, since it leads to a double free--one inside the open block, and one during the
// transactional abort caused by an AutoRTFM hazard. This can cause the unit test harness to fail on a subsequent test,
// or during shutdown. In normal operation, this doesn't matter, because AutoRTFM hazards are designed to abort.
TEST_CASE("Abort.Language.ErrorHints.MallocOpenFree")
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	const AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		void* Data = FMemory::Malloc(100);

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			FMemory::Free(Data);
		});
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	REQUIRE(WarningContext.HasWarningSubstring("Memory has been freed from the open while inside an AutoRTFM transaction."));
}
#endif

TEMPLATE_TEST_CASE("Abort.Language.SpscMpscQueue.SafeCtorDtor", "", TSpscQueue<int>, TMpscQueue<int>)
{
	SECTION("Ctor")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TestType Queue;
		});
	}

	SECTION("Dtor, Commit")
	{
		TUniquePtr<TestType> Queue = MakeUnique<TestType>();
		Queue->Enqueue(123);
		Queue->Enqueue(456);

		AutoRTFM::Testing::Commit([&]
		{
			Queue = nullptr;
		});
	}

	SECTION("Dtor, Abort")
	{
		TUniquePtr<TestType> Queue = MakeUnique<TestType>();
		Queue->Enqueue(123);
		Queue->Enqueue(456);

		AutoRTFM::Testing::Abort([&]
		{
			Queue = nullptr;
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(!Queue->IsEmpty());
	}
}

TEMPLATE_TEST_CASE("Abort.Language.ErrorHints.SpscMpscQueue", "", TSpscQueue<int>, TMpscQueue<int>)
{
	FScopedErrorHintTest ScopedTest;
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	TestType Queue;
	AutoRTFM::ETransactionResult Result;

	SECTION("Enqueue")
	{
		Result = AutoRTFM::Transact([&]
		{
			Queue.Enqueue(42);
		});
	}

	SECTION("Dequeue -> int")
	{
		Queue.Enqueue(42);

		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(Queue.Dequeue() == 42);
		});
	}

	SECTION("Dequeue(int&)")
	{
		Queue.Enqueue(42);

		Result = AutoRTFM::Transact([&]
		{
			int FortyTwo;
			Queue.Dequeue(FortyTwo);
		});
	}

	SECTION("Peek")
	{
		Queue.Enqueue(42);

		Result = AutoRTFM::Transact([&]
		{
			REQUIRE(*Queue.Peek() == 42);
		});
	}

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);

	// Did we print our message?
	const char* ExpectedMessage = "???";
	if constexpr (std::is_same_v<TestType, TSpscQueue<int>>)
	{
		ExpectedMessage = "An SPSC queue was accessed while inside an AutoRTFM transaction.";
	}
	else if constexpr (std::is_same_v<TestType, TMpscQueue<int>>)
	{
		ExpectedMessage = "An MPSC queue was accessed while inside an AutoRTFM transaction.";
	}
	else
	{
		static_assert(!"unexpected TestType");
	}
	REQUIRE(WarningContext.HasWarningSubstring(ExpectedMessage));
}
