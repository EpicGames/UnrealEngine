// Copyright Epic Games, Inc. All Rights Reserved.

#include "Allocator.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "MemoryStats.h"

#include <stdlib.h>

#ifdef __clang__
#define CLANG_BEGIN_DISABLE_OPTIMIZATIONS  _Pragma("clang optimize off")
#define CLANG_END_DISABLE_OPTIMIZATIONS    _Pragma("clang optimize on")
#else
#define CLANG_BEGIN_DISABLE_OPTIMIZATIONS
#define CLANG_END_DISABLE_OPTIMIZATIONS
#endif

#ifdef _MSC_VER
#define MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL \
		__pragma(warning(push)) \
		__pragma(warning(disable : 6011))
#define MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL  __pragma(warning(pop))
#else
#define MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL
#define MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
#endif

TEST_CASE("Allocators.New")
{
	SECTION("With Abort")
	{
		int* Data = nullptr;
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Data = new int(42);
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Data);
	}

	SECTION("With Commit")
	{
		int* Data = nullptr;
		AutoRTFM::Testing::Commit([&]
			{
				Data = new int(42);
			});

		REQUIRE(42 == *Data);

		delete Data;
	}
}

TEST_CASE("Allocators.Delete")
{
	SECTION("With Abort")
	{
		int* Data = new int(42);

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				delete Data;
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(42 == *Data);

		delete Data;
	}

	SECTION("With Commit")
	{
		int* Data = new int(42);

		AutoRTFM::Testing::Commit([&]
			{
				delete Data;
			});
	}
}

TEST_CASE("Allocators.ArrayNew")
{
	SECTION("With Abort")
	{
		int* Data = nullptr;
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Data = new int[42]();
				Data[2] = 42;
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Data);
	}

	SECTION("With Commit")
	{
		int* Data = nullptr;
		AutoRTFM::Testing::Commit([&]
			{
				Data = new int[42]();
				Data[2] = 42;
			});

		REQUIRE(nullptr != Data);
		REQUIRE(42 == Data[2]);

		delete[] Data;
	}
}

TEST_CASE("Allocators.ArrayDelete")
{
	SECTION("With Abort")
	{
		int* Data = new int[42]();
		Data[10] = 10;

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				delete[] Data;
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(10 == Data[10]);
		delete[] Data;
	}

	SECTION("With Commit")
	{
		int* Data = new int[42]();

		AutoRTFM::Testing::Commit([&]
			{
				delete[] Data;
			});
	}
}

TEST_CASE("Allocators.NewNoOpts")
{
	SECTION("With Abort")
	{
		int* Data = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Data = new int(42);
				AutoRTFM::AbortTransaction();
			});

		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Data);
	}

	SECTION("With Commit")
	{
		int* Data = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS

		AutoRTFM::Testing::Commit([&]
			{
				Data = new int(42);
			});

		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(42 == *Data);

		delete Data;
	}
}

TEST_CASE("Allocators.DeleteNoOpts")
{
	SECTION("With Abort")
	{
		int* Data = new int(42);

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				delete Data;
				AutoRTFM::AbortTransaction();
			});

		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(42 == *Data);
		delete Data;
	}

	SECTION("With Commit")
	{
		int* Data = new int(42);

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS

		AutoRTFM::Testing::Commit([&]
			{
				delete Data;
			});

		CLANG_END_DISABLE_OPTIMIZATIONS
	}
}

TEST_CASE("Allocators.Free")
{
	MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL

	SECTION("With Abort")
	{
		void* Data = malloc(32);
		*reinterpret_cast<int*>(Data) = 42;

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				free(Data);
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(42 == *reinterpret_cast<int*>(Data));

		free(Data);
	}

	SECTION("With Commit")
	{
		void* Data = malloc(32);

		AutoRTFM::Testing::Commit([&]
			{
				free(Data);
			});
	}

	MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
}

TEST_CASE("Allocators.Malloc")
{
	MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL

	SECTION("With Abort")
	{
		void* Data = nullptr;
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Data = malloc(32);
				*reinterpret_cast<int*>(Data) = 42;
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Data);
	}

	SECTION("With Commit")
	{
		void* Data = nullptr;
		AutoRTFM::Testing::Commit([&]
			{
				Data = malloc(32);
				*reinterpret_cast<int*>(Data) = 42;
			});

		REQUIRE(nullptr != Data);
		REQUIRE(42 == *reinterpret_cast<int*>(Data));

		free(Data);
	}

	MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
}

TEST_CASE("Allocators.Calloc")
{
	MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL

	SECTION("With Abort")
	{
		void* Data = nullptr;
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Data = calloc(5, sizeof(int));
				*reinterpret_cast<int*>(Data) = 42;
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Data);
	}

	SECTION("With Commit")
	{
		void* Data = nullptr;
		AutoRTFM::Testing::Commit([&]
			{
				Data = calloc(5, sizeof(int));
				*reinterpret_cast<int*>(Data) = 42;
			});

		REQUIRE(nullptr != Data);
		REQUIRE(42 == reinterpret_cast<int*>(Data)[0]);
		REQUIRE(0 == reinterpret_cast<int*>(Data)[1]);

		free(Data);
	}

	MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
}

TEST_CASE("Allocators.Realloc")
{
	MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL

	SECTION("With Abort")
	{
		void* Alloc = malloc(32);
		void* Realloc = nullptr;
		*reinterpret_cast<int*>(Alloc) = 42;

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Realloc = realloc(Alloc, 64);
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Realloc);
		REQUIRE(42 == *reinterpret_cast<int*>(Alloc));

		free(Alloc);
	}

	SECTION("With Commit")
	{
		void* Alloc = malloc(32);
		void* Realloc = nullptr;
		*reinterpret_cast<int*>(Alloc) = 42;

		AutoRTFM::Testing::Commit([&]
			{
				Realloc = realloc(Alloc, 64);
			});

		REQUIRE(nullptr != Realloc);
		REQUIRE(42 == *reinterpret_cast<int*>(Realloc));

		free(Realloc);
	}

	MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
}

TEST_CASE("Allocators.MemoryStats.GetTotalBytesAllocated")
{
	size_t StartingSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();

	void* const Allocate1MBytes = AutoRTFM::FAllocator::Allocate(1'000'000, /*Alignment*/ 1);
	size_t CurrentSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();
	REQUIRE(StartingSize + 1'000'000 == CurrentSize);

	void* const Allocate49MBytes = AutoRTFM::FAllocator::AllocateZeroed(49'000'000, /*Alignment*/ 1);
	CurrentSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();
	REQUIRE(StartingSize + 50'000'000 == CurrentSize);

	AutoRTFM::FAllocator::Free(Allocate1MBytes, 1'000'000);
	CurrentSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();
	REQUIRE(StartingSize + 49'000'000 == CurrentSize);

	void* const Allocate7MBytes = AutoRTFM::FAllocator::Reallocate(Allocate49MBytes, 7'000'000, /*Alignment*/ 1, /*PreviousSize*/ 49'000'000);
	CurrentSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();
	REQUIRE(StartingSize + 7'000'000 == CurrentSize);

	AutoRTFM::FAllocator::Free(Allocate7MBytes, 7'000'000);
	CurrentSize = AutoRTFM::FMemoryStats::GetTotalBytesAllocated();
	REQUIRE(CurrentSize == StartingSize);
}

#undef CLANG_BEGIN_DISABLE_OPTIMIZATIONS
#undef CLANG_END_DISABLE_OPTIMIZATIONS
#undef MSVC_BEGIN_DISABLE_WARN_ALLOC_MAYBE_NULL
#undef MSVC_END_DISABLE_WARN_ALLOC_MAYBE_NULL
