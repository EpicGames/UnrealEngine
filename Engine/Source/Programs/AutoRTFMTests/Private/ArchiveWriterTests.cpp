// Copyright Epic Games, Inc. All Rights Reserved.

#include "Catch2Includes.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Misc/NotNull.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/TransactionallySafeArchiveWriter.h"

/**
 * This test class wraps an FMemoryWriter, but check-fails if Serialize is ever called
 * from within a transaction. This will cause our tests to fail if we violate the
 * transactional boundary.
 */
class FCheckingMemoryWriter : public FMemoryWriter
{
	using Super = FMemoryWriter;

public:
	FCheckingMemoryWriter(TArray<uint8>& Buffer, TNotNull<int*> InNumFlushes) 
		: Super(Buffer)
		, NumFlushes(*InNumFlushes)
	{
		NumFlushes = 0;
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FCheckingMemoryWriter");
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		check(!AutoRTFM::IsTransactional());
		Super::Serialize(Data, Num);
	}

	virtual void Flush() override
	{
		++NumFlushes;
	}
	
	int& NumFlushes;
};

static void SerializeData(FArchive& Archive)
{
	// Serializes various data into the archive: short, packed int, bool, bit-array, string.
	uint16 Short = 123;
	Archive << Short;
	uint32 PackedUint = 4567;
	Archive.SerializeIntPacked(PackedUint);
	bool Boolean = true;
	Archive << Boolean;
	uint32 Bits = 0x55555555;
	Archive.SerializeBits(&Bits, 11);
	FString("Hello").SerializeAsANSICharArray(Archive);
}

TEST_CASE("TransactionallySafeArchiveWriter.MatchesMemoryWriter")
{
	// This test verifies that a variety of serialization methods (via SerializeData) work as designed.

	TArray<uint8> NormalStorage;
	FMemoryWriter MemoryWriter(NormalStorage);
	SerializeData(MemoryWriter);

	SECTION("Write")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
		SerializeData(SafeWriter);

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Flush")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
		SafeWriter.Flush();

		REQUIRE(NumFlushes == 1);
	}

	SECTION("Ctor, Commit(Write)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			SerializeData(SafeWriter);

			// The FTransactionallySafeArchiveWriter must defer writes into the backing storage.
			REQUIRE(TSStorage.IsEmpty());
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Commit(Ctor, Write)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
			SerializeData(SafeWriter);

			// The FTransactionallySafeArchiveWriter must defer writes into the backing storage.
			REQUIRE(TSStorage.IsEmpty());
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Ctor, Commit(Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			SafeWriter.Flush();

			// The FTransactionallySafeArchiveWriter must defer flushes on the backing storage.
			REQUIRE(NumFlushes == 0);
		});

		REQUIRE(TSStorage.IsEmpty());
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Commit(Ctor, Flush, Flush, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
			SafeWriter.Flush();
			SafeWriter.Flush();
			SafeWriter.Flush();

			// The FTransactionallySafeArchiveWriter must defer flushes on the backing storage.
			REQUIRE(NumFlushes == 0);
		});

		REQUIRE(TSStorage.IsEmpty());
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Ctor, Commit(Write, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			SerializeData(SafeWriter);
			SafeWriter.Flush();

			// The FTransactionallySafeArchiveWriter must defer writes and flushes to the backing storage.
			REQUIRE(TSStorage.IsEmpty());
			REQUIRE(NumFlushes == 0);
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Commit(Ctor, Write, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
			SerializeData(SafeWriter);
			SafeWriter.Flush();

			// The FTransactionallySafeArchiveWriter must defer writes and flushes to the backing storage.
			REQUIRE(TSStorage.IsEmpty());
			REQUIRE(NumFlushes == 0);
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}
}

TEST_CASE("TransactionallySafeArchiveWriter.CommitAndAbortWork")
{
	// This test verifies that different patterns of construction, committing, and aborting
	// all yield the expected result.

	TArray<uint8> NormalStorage;
	FMemoryWriter MemoryWriter(NormalStorage);
	FString("Hello").SerializeAsANSICharArray(MemoryWriter);

	SECTION("Ctor, Write, Abort(Write, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		FString("Hello").SerializeAsANSICharArray(SafeWriter);

		AutoRTFM::Testing::Abort([&]
		{
			FString("World").SerializeAsANSICharArray(SafeWriter);
			SafeWriter.Flush();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Ctor, Commit(Write, Abort(Write), Abort(Flush))")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			FString("Hello").SerializeAsANSICharArray(SafeWriter);

			AutoRTFM::Testing::Abort([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				AutoRTFM::AbortTransaction();
			});

			AutoRTFM::Testing::Abort([&]
			{
				SafeWriter.Flush();
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Commit(Ctor, Write, Abort(Write, Flush))")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
			FString("Hello").SerializeAsANSICharArray(SafeWriter);

			AutoRTFM::Testing::Abort([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				SafeWriter.Flush();
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Ctor, Commit(Abort(Write), Write, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Abort([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				AutoRTFM::AbortTransaction();
			});

			FString("Hello").SerializeAsANSICharArray(SafeWriter);
			SafeWriter.Flush();
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Commit(Ctor, Abort(Write), Write, Flush)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

			AutoRTFM::Testing::Abort([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				AutoRTFM::AbortTransaction();
			});

			FString("Hello").SerializeAsANSICharArray(SafeWriter);
			SafeWriter.Flush();
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Commit(Ctor, Abort(Write, Flush), Write)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

			AutoRTFM::Testing::Abort([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				SafeWriter.Flush();
				AutoRTFM::AbortTransaction();
			});

			FString("Hello").SerializeAsANSICharArray(SafeWriter);
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}

	SECTION("Ctor, Write, Abort(Commit(Write, Flush)))")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		FString("Hello").SerializeAsANSICharArray(SafeWriter);

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::Testing::Commit([&]
			{
				FString("World").SerializeAsANSICharArray(SafeWriter);
				SafeWriter.Flush();
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 0);
	}
}

TEST_CASE("TransactionallySafeArchiveWriter.MixingWrites")
{
	// This test verifies that an archive writes can be mixed between the transactionally-safe
	// writer and its wrapped archive, using Release() to take the archive back as needed.

	TArray<uint8> NormalStorage;
	FMemoryWriter MemoryWriter(NormalStorage);
	FString("Hello").SerializeAsANSICharArray(MemoryWriter);
	FString("World").SerializeAsANSICharArray(MemoryWriter);

	SECTION("Wrap, Write, Release, Write")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));
		FString("Hello").SerializeAsANSICharArray(SafeWriter);

		TUniquePtr<FArchive> ReleasedWriter = SafeWriter.Release();
		FString("World").SerializeAsANSICharArray(*ReleasedWriter);

		REQUIRE(TSStorage == NormalStorage);
	}

	SECTION("Wrap, Commit(Write), Release, Write")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			FString("Hello").SerializeAsANSICharArray(SafeWriter);
		});

		TUniquePtr<FArchive> ReleasedWriter = SafeWriter.Release();
		FString("World").SerializeAsANSICharArray(*ReleasedWriter);

		REQUIRE(TSStorage == NormalStorage);
	}

	SECTION("Write, Wrap, Commit(Write), Release")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		TUniquePtr<FCheckingMemoryWriter> CheckingWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		FString("Hello").SerializeAsANSICharArray(*CheckingWriter);

		FTransactionallySafeArchiveWriter SafeWriter(MoveTemp(CheckingWriter));

		AutoRTFM::Testing::Commit([&]
		{
			FString("World").SerializeAsANSICharArray(SafeWriter);
		});

		REQUIRE(TSStorage == NormalStorage);

		SafeWriter.Release();

		REQUIRE(TSStorage == NormalStorage);
	}

	SECTION("Write, Wrap, Release, Write")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		TUniquePtr<FCheckingMemoryWriter> CheckingWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		FString("Hello").SerializeAsANSICharArray(*CheckingWriter);

		FTransactionallySafeArchiveWriter SafeWriter(MoveTemp(CheckingWriter));
		TUniquePtr<FArchive> ReleasedWriter = SafeWriter.Release();

		FString("World").SerializeAsANSICharArray(*ReleasedWriter);

		REQUIRE(TSStorage == NormalStorage);
	}
}

TEST_CASE("TransactionallySafeArchiveWriter.OnCommitWorks")
{
	TArray<uint8> NormalStorage;
	FMemoryWriter MemoryWriter(NormalStorage);
	FString("One").SerializeAsANSICharArray(MemoryWriter);
	FString("Two").SerializeAsANSICharArray(MemoryWriter);
	FString("Three").SerializeAsANSICharArray(MemoryWriter);

	SECTION("Commit(OnCommit)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnCommit([&]
			{
				FString("One").SerializeAsANSICharArray(SafeWriter);
				FString("Two").SerializeAsANSICharArray(SafeWriter);
				FString("Three").SerializeAsANSICharArray(SafeWriter);
				SafeWriter.Flush();
			});
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}

	SECTION("Write, Commit(Write, OnCommit)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		FString("One").SerializeAsANSICharArray(SafeWriter);

		AutoRTFM::Testing::Commit([&]
		{
			FString("Two").SerializeAsANSICharArray(SafeWriter);

			AutoRTFM::OnCommit([&]
			{
				FString("Three").SerializeAsANSICharArray(SafeWriter);
			});
		});

		REQUIRE(TSStorage == NormalStorage);
	}

	SECTION("Commit(OnCommit, Write), Write")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnCommit([&]
			{
				FString("Two").SerializeAsANSICharArray(SafeWriter);
			});

			FString("One").SerializeAsANSICharArray(SafeWriter);
		});

		FString("Three").SerializeAsANSICharArray(SafeWriter);

		REQUIRE(TSStorage == NormalStorage);
	}

	SECTION("Commit(OnCommit, Write, OnCommit, OnCommit)")
	{
		TArray<uint8> TSStorage;
		int NumFlushes = 0;
		FTransactionallySafeArchiveWriter SafeWriter(MakeUnique<FCheckingMemoryWriter>(TSStorage, &NumFlushes));

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnCommit([&]
			{
				FString("Two").SerializeAsANSICharArray(SafeWriter);
			});

			FString("One").SerializeAsANSICharArray(SafeWriter);

			AutoRTFM::OnCommit([&]
			{
				FString("Three").SerializeAsANSICharArray(SafeWriter);
			});

			AutoRTFM::OnCommit([&]
			{
				SafeWriter.Flush();
			});
		});

		REQUIRE(TSStorage == NormalStorage);
		REQUIRE(NumFlushes == 1);
	}
}
