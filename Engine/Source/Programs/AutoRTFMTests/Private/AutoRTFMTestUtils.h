// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "AutoRTFM/Defines.h"
#include "Containers/StringConv.h"
#include "HAL/Platform.h"
#include "Logging/StructuredLog.h"
#include "Misc/FeedbackContext.h"

#include <algorithm>
#include <string>
#include <vector>

namespace AutoRTFMTestUtils
{

// Temporarily changes the AutoRTFM retry mode for the lifetime of the FScopedRetry object.
struct FScopedRetry
{
	FScopedRetry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState NewRetry)
		: OldRetry(AutoRTFM::ForTheRuntime::GetRetryTransaction())
	{
		AutoRTFM::ForTheRuntime::SetRetryTransaction(NewRetry);
	}

	~FScopedRetry()
	{
		AutoRTFM::ForTheRuntime::SetRetryTransaction(OldRetry);
	}

	AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState OldRetry;
};

// A helper class that for the lifetime of the object, intercepts and records UE_LOG warnings.
class FCaptureWarningContext : private FFeedbackContext
{
public:
	FCaptureWarningContext() : OldContext(GWarn) { GWarn = this; }
	~FCaptureWarningContext() { GWarn = OldContext; }

	UE_AUTORTFM_ALWAYS_OPEN
	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		if (Verbosity == ELogVerbosity::Warning)
		{
			Warnings.push_back(std::string(StringCast<ANSICHAR>(V).Get()));
		}
		else
		{
			OldContext->Serialize(V, Verbosity, Category);
		}
	}

	UE_AUTORTFM_ALWAYS_OPEN
	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override
	{
		if (Verbosity == ELogVerbosity::Warning)
		{
			Warnings.push_back(std::string(StringCast<ANSICHAR>(V).Get()));
		}
		else
		{
			OldContext->Serialize(V, Verbosity, Category, Time);
		}
	}

	UE_AUTORTFM_ALWAYS_OPEN
	void SerializeRecord(const UE::FLogRecord& Record) override
	{
		if (Record.GetVerbosity() == ELogVerbosity::Warning)
		{
			// Skip FFeedbackContext and allow the base impl to turn the record into a string so we can store it
			FOutputDevice::SerializeRecord(Record);
		}
		else
		{
			OldContext->SerializeRecord(Record);
		}
	}

	UE_AUTORTFM_ALWAYS_OPEN
	const std::vector<std::string>& GetWarnings() const
	{
		return Warnings;
	}

	UE_AUTORTFM_ALWAYS_OPEN
	bool HasWarning(std::string_view Expected) const
	{
		return std::any_of(Warnings.begin(), Warnings.end(), [&](std::string_view Warning)
		{
			return Warning == Expected;
		});
	}

	UE_AUTORTFM_ALWAYS_OPEN
	bool HasWarningSubstring(std::string_view Substr) const
	{
		return std::any_of(Warnings.begin(), Warnings.end(), [&](std::string_view Warning)
		{
			return Warning.find(Substr) != std::string::npos;
		});
	}

private:
	FCaptureWarningContext(const FCaptureWarningContext&) = delete;
	FCaptureWarningContext& operator = (const FCaptureWarningContext&) = delete;

	FFeedbackContext* OldContext = nullptr;
	std::vector<std::string> Warnings;
};

#if AUTORTFM_SANITIZER
// A helper class that temporarily changes the AutoRTFM sanitizer mode for the
// lifetime of the object, restoring the original level on destruction.
class FScopedSanitizerMode
{
public:
	FScopedSanitizerMode(AutoRTFM::Sanitizer::EMode NewMode)
		: PrevMode(AutoRTFM::Sanitizer::GetMode())
	{
		AutoRTFM::Sanitizer::SetMode(NewMode);
	}

	~FScopedSanitizerMode()
	{
		AutoRTFM::Sanitizer::SetMode(PrevMode);
	}
private:
	FScopedSanitizerMode(const FScopedSanitizerMode&) = delete;
	FScopedSanitizerMode& operator = (const FScopedSanitizerMode&) = delete;

	const AutoRTFM::Sanitizer::EMode PrevMode;
};
#endif // AUTORTFM_SANITIZER

#define AUTORTFM_SCOPED_DISABLE_RETRY() \
	AutoRTFMTestUtils::FScopedRetry ScopedDisableRetry \
		{AutoRTFM::ForTheRuntime::NoRetry}

#if AUTORTFM_SANITIZER
#define AUTORTFM_SCOPED_SANITIZER_MODE_DISABLE() \
	AutoRTFMTestUtils::FScopedSanitizerMode ScopedSanitizerModeWarning \
		{AutoRTFM::Sanitizer::EMode::Disabled}
#define AUTORTFM_SCOPED_SANITIZER_MODE_WARN() \
	AutoRTFMTestUtils::FScopedSanitizerMode ScopedSanitizerModeWarning \
		{AutoRTFM::Sanitizer::EMode::Warn}
#else
#define AUTORTFM_SCOPED_SANITIZER_MODE_DISABLE() static_assert(true) /* require semicolon */
#endif

static constexpr const std::string_view kMemoryModifiedInOpenWarning =
	"This may lead to memory corruption if the transaction is aborted.";

class FScopedEnsureOnInternalAbort final
{
public:
	FScopedEnsureOnInternalAbort(const bool bState)
		: bOriginal(AutoRTFM::ForTheRuntime::GetEnsureOnInternalAbort())
	{
		AutoRTFM::ForTheRuntime::SetEnsureOnInternalAbort(bState);
	}

	~FScopedEnsureOnInternalAbort()
	{
		AutoRTFM::ForTheRuntime::SetEnsureOnInternalAbort(bOriginal);
	}

private:
	FScopedEnsureOnInternalAbort(const FScopedEnsureOnInternalAbort&) = delete;
	FScopedEnsureOnInternalAbort& operator = (const FScopedEnsureOnInternalAbort&) = delete;

	const bool bOriginal;
};

class FScopedInternalAbortAction final
{
public:
	FScopedInternalAbortAction(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState State)
		: Original(AutoRTFM::ForTheRuntime::GetInternalAbortAction())
	{
		AutoRTFM::ForTheRuntime::SetInternalAbortAction(State);
	}

	~FScopedInternalAbortAction()
	{
		AutoRTFM::ForTheRuntime::SetInternalAbortAction(Original);
	}

private:
	FScopedInternalAbortAction(const FScopedInternalAbortAction&) = delete;
	FScopedInternalAbortAction& operator = (const FScopedInternalAbortAction&) = delete;

	const AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState Original;
};

}
