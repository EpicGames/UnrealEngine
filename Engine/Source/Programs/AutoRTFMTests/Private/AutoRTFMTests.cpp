// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFM/AutoRTFMTestingUE.h"
#include "CoreMinimal.h"
#include "Misc/EventPool.h"
#include "Misc/LazySingleton.h"
#include "Misc/CString.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "RequiredProgramMainCPPInclude.h" // required for ue programs
#include "UObject/GCObject.h"
#include "Serialization/PackageStore.h"

IMPLEMENT_APPLICATION(AutoRTFMTests, "AutoRTFMTests");

#define CATCH_AMALGAMATED_CUSTOM_MAIN
#include "catch_amalgamated.cpp"

void AutoRTFM::Testing::AssertionFailure(const char* Expression, const char* File, int Line)
{
	FAIL(File << ":" << Line << ": " << Expression);
}

int RunTests(int ArgC, const char* ArgV[])
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	Catch::Session Session;

	bool bNoRetry = false;
	bool bRetryNestedToo = false;
	bool bEnableBenchmarks = false;
	AUTORTFM_SANITIZER_ONLY(bool bSanitizerRecordWriteCallstacks = false);

	Session.cli(Session.cli()
		| Catch::Clara::Opt(bNoRetry)["--no-retry"]
		| Catch::Clara::Opt(bRetryNestedToo)["--retry-nested-too"]
		| Catch::Clara::Opt(bEnableBenchmarks)["--enable-benchmarks"]
#if AUTORTFM_SANITIZER
		| Catch::Clara::Opt(bSanitizerRecordWriteCallstacks)["--sanitizer-record-write-callstacks"]
#endif
	);

	TStringBuilder<256> CommandLine(InPlace,
		TEXT("-Multiprocess -csvNoProcessingThread -LogCmds=\"LogCsvProfiler off, LogStreaming off, LogUObjectGlobals off, LogPackageName off, LogAutoRTFM warning\" -AsyncLoadingThread"));
#if FORCE_USE_STATS
	CommandLine << TEXT(" -LoadTimeStatsForCommandlet");
#endif
	// Allow callers to pass "... -- <UnrealArgs>" on the commandline
	int OriginalArgC = ArgC;
	for (int N = 0; N < OriginalArgC; ++N)
	{
		if (0 == FCStringAnsi::Strcmp(ArgV[N], "--"))
		{
			ArgC = N;
			for (int CopiedIndex = N + 1; CopiedIndex < OriginalArgC; ++CopiedIndex)
			{
				CommandLine << TEXT(" ") << ArgV[CopiedIndex];
			}
			break;
		}
	}
	
	{
		const int Result = Session.applyCommandLine(ArgC, ArgV);

		if (!bEnableBenchmarks)
		{
			constexpr int FakeArgC = 2;
			const char* const FakeArgV[FakeArgC] = { ArgV[0], "--skip-benchmarks" };
			Session.applyCommandLine(FakeArgC, FakeArgV);
		}

		if (0 != Result)
		{
			return Result;
		}
	}

	AUTORTFM_SANITIZER_ONLY(AutoRTFM::Sanitizer::SetRecordClosedWriteCallstacks(bSanitizerRecordWriteCallstacks));

	GEngineLoop.PreInit(*CommandLine); // Note: Initializes AutoRTFM
	GLog->SetCurrentThreadAsPrimaryThread();

	FCsvProfiler::Get()->Init();

	if (bRetryNestedToo)
	{
		AutoRTFM::ForTheRuntime::SetRetryTransaction(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::RetryNestedToo);
	}
	else if (bNoRetry)
	{
		AutoRTFM::ForTheRuntime::SetRetryTransaction(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);
	}
	else
	{
		// Otherwise default to just retrying the parent transaction.
		AutoRTFM::ForTheRuntime::SetRetryTransaction(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::RetryNonNested);
	}

	// Initialize the garbage collector. (Otherwise, if no active test uses garbage collection, 
	// the call to `CollectGarbage` below might fail.)
	FGCObject::StaticInit();

	// Enable AutoRTFM.
	AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault);

	// By default, crash on an internal abort to keep testing honest.
	AutoRTFM::ForTheRuntime::SetInternalAbortAction(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash);

	// A fake package store backend so we can actually load async packages for testing.
	struct MyPackageStoreBackend final : IPackageStoreBackend
	{
		void OnMounted(TSharedRef<const FPackageStoreBackendContext> InContext) override {} 
		void BeginRead() override {}
		void EndRead() override {} 

		EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName,
			FPackageStoreEntry& OutPackageStoreEntry) override
		{
			if (!FPackageName::DoesPackageExist(PackageName.ToString()))
			{
				return EPackageStoreEntryStatus::Missing;
			}

			OutPackageStoreEntry.LoaderType = EPackageLoader::LinkerLoad;
			OutPackageStoreEntry.LinkerLoadCaseCorrectedPackageName = PackageName;
			
			return EPackageStoreEntryStatus::Ok;
		}

		bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override
		{
			return false;
		}
		
		EPackageLoader GetSupportedLoaders() override
		{
			return EPackageLoader::Zen | EPackageLoader::LinkerLoad;
		}
	};

	TSharedPtr<MyPackageStoreBackend> PackageStoreBackend = MakeShared<MyPackageStoreBackend>();
	FPackageStore::Get().Mount(PackageStoreBackend.ToSharedRef());

	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	int Result = Session.run();

	FPlatformMisc::RequestExit(false);

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, /* bPerformFullPurge */ true);

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();
	TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().EmptyPool();
	TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().EmptyPool();

	return Result;
}

AUTORTFM_DISABLE int main(int ArgC, const char* ArgV[])
{
#if AUTORTFM_ENABLE_TEST_UTILS
	AutoRTFM::Testing::PrintCallstackOnCrash();

	// Install the AutoRTFM memory tracking allocator to test heap allocation correctness.
	constexpr bool bRecordAllocationStackTraces = false;
	AutoRTFM::Testing::FTrackingAllocator AutoRTFMTrackingAllocator([](const char* Error) {
		ensureMsgf(false, TEXT("AutoRTFM tracking allocator failure: %hs"), Error);
	});
	AutoRTFM::Testing::InterceptExternAPIs([&](autortfm_extern_api& API)
	{
		AutoRTFMTrackingAllocator.Install(API, bRecordAllocationStackTraces);
	});
#endif // AUTORTFM_ENABLE_TEST_UTILS

	const int Result = RunTests(ArgC, ArgV);

#define CASE_CATCH_EXIT_CODE(X)														   \
		case Catch::X:																   \
			ensureMsgf(false, TEXT("AutoRTFM tests failed with ") TEXT(#X) TEXT(".")); \
			return Result;

	switch (Result)
	{
		CASE_CATCH_EXIT_CODE(UnspecifiedErrorExitCode)
		CASE_CATCH_EXIT_CODE(NoTestsRunExitCode)
		CASE_CATCH_EXIT_CODE(UnmatchedTestSpecExitCode)
		CASE_CATCH_EXIT_CODE(AllTestsSkippedExitCode)
		CASE_CATCH_EXIT_CODE(InvalidTestSpecExitCode)
		CASE_CATCH_EXIT_CODE(TestFailureExitCode)

		case 0:  // success
			break;

		default:
			ensureMsgf(false, TEXT("AutoRTFM tests failed with code %d"), Result);
			return Result;
	}

#undef CASE_CATCH_EXIT_CODE

#if AUTORTFM_ENABLE_TEST_UTILS
	if (size_t NumAllocated = AutoRTFMTrackingAllocator.TotalBytesAllocated(); NumAllocated != 0)
	{
		// Change bRecordAllocationStackTraces to true to see leaked memory callstacks.
		AutoRTFMTrackingAllocator.PrintAllocationCallstacks();
		ensureMsgf(false, TEXT("AutoRTFM leaked %zu bytes"), NumAllocated);
		return -1;
	}
#endif // AUTORTFM_ENABLE_TEST_UTILS

	return FDebug::GetNumEnsureFailures() == 0 ? 0 : -1;
}
