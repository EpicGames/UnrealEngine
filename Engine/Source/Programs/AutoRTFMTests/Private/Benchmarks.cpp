// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "Utils.h"
#include "Catch2Includes.h"

#include <vector>
#include <unordered_set>

using namespace std;

template<typename T>
class FNode
{
public:
	FNode(T Value) : Value(Value) { }

	T GetValue() const { return Value; }

	void AddEdge(FNode* Target)
	{
		Edges.push_back(Target);
	}

	const vector<FNode*>& GetEdges() const { return Edges; }

private:
	vector<FNode*> Edges;
	T Value;
};

template<typename T>
class FGraph
{
public:
	FGraph() = default;

	FGraph(const FGraph&) = delete;
	FGraph& operator=(const FGraph&) = delete;

	FGraph(FGraph&& InGraph)
	{
		*this = std::move(InGraph);
	}

	FGraph& operator=(FGraph&& InGraph)
	{
		Reset();
		Nodes = std::move(InGraph.Nodes);
		Roots = std::move(InGraph.Roots);
		return *this;
	}

	~FGraph()
	{
		Reset();
	}

	void Reset()
	{
		for (FNode<T>* Node : Nodes)
		{
			delete Node;
		}
		Nodes.clear();
		Roots.clear();
	}

	FNode<T>* AddNode(T Value)
	{
		FNode<T>* Node = new FNode<T>(Value);
		Nodes.push_back(Node);
		return Node;
	}

	void AddRoot(FNode<T>* Node)
	{
		Roots.push_back(Node);
	}

	const vector<FNode<T>*>& GetNodes() const { return Nodes; }
	const vector<FNode<T>*>& GetRoots() const { return Roots; }

	template<typename TFunc>
	void DepthFirstSearchPre(const TFunc& Func) const
	{
		vector<FNode<T>*> Worklist;
		unordered_set<FNode<T>*> Seen;

		auto Push = [&](FNode<T>* Node)
			{
				if (Seen.insert(Node).second)
				{
					Worklist.push_back(Node);
				}
			};

		auto PushAll = [&](const vector<FNode<T>*>& InnerNodes)
			{
				for (FNode<T>* Node : InnerNodes)
				{
					Push(Node);
				}
			};

		PushAll(Roots);

		while (!Worklist.empty())
		{
			FNode<T>* Node = Worklist.back();
			Worklist.pop_back();

			Func(Node);

			PushAll(Node->GetEdges());
		}
	}

private:
	vector<FNode<T>*> Nodes;
	vector<FNode<T>*> Roots;
};

unsigned XorshiftState;

void ResetXorshift()
{
	XorshiftState = 666;
}

unsigned Xorshift()
{
	unsigned Value = XorshiftState;
	Value ^= Value << 13;
	Value ^= Value >> 17;
	Value ^= Value << 5;
	XorshiftState = Value;
	return Value;
}

unsigned BadRandom(unsigned Limit)
{
	return Xorshift() % Limit; // Yes I know this isn't great.
}

void AddToGraph(FGraph<unsigned>& Graph)
{
	auto RandomNode = [&](unsigned Value)
		{
			FNode<unsigned>* Result = Graph.AddNode(Value);
			if (!BadRandom(Graph.GetNodes().size() + 1))
			{
				Graph.AddRoot(Result);
			}
			return Result;
		};

	FNode<unsigned>* A = RandomNode(Xorshift());
	FNode<unsigned>* B = RandomNode(Xorshift());
	FNode<unsigned>* C = RandomNode(Xorshift());
	FNode<unsigned>* D = RandomNode(Xorshift());
	FNode<unsigned>* E = RandomNode(Xorshift());
	FNode<unsigned>* F = RandomNode(Xorshift());
	FNode<unsigned>* G = RandomNode(Xorshift());

	auto RandomEdge = [&](FNode<unsigned>* From, FNode<unsigned>* To)
		{
			if (!BadRandom(5))
			{
				From = Graph.GetNodes()[BadRandom(Graph.GetNodes().size())];
			}
			if (!BadRandom(5))
			{
				To = Graph.GetNodes()[BadRandom(Graph.GetNodes().size())];
			}
			From->AddEdge(To);
		};

	RandomEdge(A, B);
	RandomEdge(B, C);
	RandomEdge(C, B);
	RandomEdge(C, D);
	RandomEdge(B, D);
	RandomEdge(D, E);
	RandomEdge(E, F);
	RandomEdge(E, G);
	RandomEdge(A, E);
	RandomEdge(A, G);
	RandomEdge(G, E);
}

FGraph<unsigned> BuildGraph(const unsigned Total = 1000)
{
	FGraph<unsigned> Result;
	for (unsigned Count = Total; Count--;)
	{
		AddToGraph(Result);
	}
	return Result;
}

unsigned WalkGraph(const FGraph<unsigned>& Graph)
{
	unsigned Result = 0;
	unsigned Count = 0;
	Graph.DepthFirstSearchPre([&](FNode<unsigned>* Node)
		{
			Result += Node->GetValue();
			Count++;
		});
	//printf("Saw %zu roots, %u nodes, sum is %u.\n", Graph.GetRoots().size(), Count, Result);

	return Result;
}

void CheckResult(unsigned Result, unsigned Total)
{
	switch (Total)
	{
	case 10000: REQUIRE(Result == 434344629); break;
	case 100: REQUIRE(Result == 3732096243); break;
	case 10: REQUIRE(Result == 3524276090); break;
	case 1: REQUIRE(Result == 2218159753); break;
	default: abort();
	}
}

TEST_CASE("Graph")
{
	{
		FGraph<unsigned> Graph;

		AutoRTFM::Testing::Commit([&]() { Graph.AddNode(42); });

		auto& Nodes = Graph.GetNodes();

		REQUIRE(Nodes.size() == 1);
		REQUIRE(Nodes[0]->GetValue() == 42);
	}

	{
		FGraph<unsigned> Graph;

		AutoRTFM::Testing::Commit([&]() { Graph.AddRoot(Graph.AddNode(42)); });

		auto& Roots = Graph.GetRoots();

		REQUIRE(Roots.size() == 1);
		REQUIRE(Roots[0]->GetValue() == 42);

		auto& Nodes = Graph.GetNodes();

		REQUIRE(Nodes.size() == 1);
		REQUIRE(Nodes[0]->GetValue() == 42);
	}

	for (unsigned Total : {1, 10, 100, 10000})
	{
#if defined(__has_feature)
#if __has_feature(address_sanitizer)
		// This test is hella long with ASan so we bail after the first iteration.
		if (Total > 1)
		{
			return;
		}
#endif
#endif

		{
			unsigned Result;
			ResetXorshift();
			FGraph<unsigned> Graph;
			AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(Total); });
			Result = WalkGraph(Graph);
			CheckResult(Result, Total);
		}

		{
			unsigned Result;
			ResetXorshift();
			FGraph<unsigned> Graph = BuildGraph(Total);
			AutoRTFM::Testing::Commit([&]() { Result = WalkGraph(Graph); });
			CheckResult(Result, Total);
		}

		{
			unsigned Result;
			ResetXorshift();
			FGraph<unsigned> Graph;
			AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(Total); });
			AutoRTFM::Testing::Commit([&]() { Result = WalkGraph(Graph); });
			CheckResult(Result, Total);
		}

		{
			unsigned Result;
			ResetXorshift();
			FGraph<unsigned> Graph;
			AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(Total); Result = WalkGraph(Graph); });
			CheckResult(Result, Total);
		}
	}

	BENCHMARK("build non transactional / walk non transactional")
	{
		ResetXorshift();
		FGraph<unsigned> Graph = BuildGraph();
		WalkGraph(Graph);
	};

	BENCHMARK("build transactional / walk non transactional")
	{
		ResetXorshift();
		FGraph<unsigned> Graph;
		AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(); });
		WalkGraph(Graph);
	};

	BENCHMARK("build non transactional / walk transactional")
	{
		ResetXorshift();
		FGraph<unsigned> Graph = BuildGraph();
		AutoRTFM::Testing::Commit([&]() { WalkGraph(Graph); });
	};

	BENCHMARK("build transactional / walk transactional")
	{
		ResetXorshift();
		FGraph<unsigned> Graph;
		AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(); });
		AutoRTFM::Testing::Commit([&]() { WalkGraph(Graph); });
	};

	BENCHMARK("build + walk transactional")
	{
		ResetXorshift();
		FGraph<unsigned> Graph;
		AutoRTFM::Testing::Commit([&]() { Graph = BuildGraph(); WalkGraph(Graph); });
	};
}

TEST_CASE("Benchmarks.PopOnAbortHandler")
{
	constexpr unsigned Count = 16 * 128;

	BENCHMARK("PopOnAbortSingleKey")
	{
		bool bHit = false;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHit, [&] { bHit = true; });
				AutoRTFM::PopOnAbortHandler(&bHit);
			}

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bHit);
	};

	BENCHMARK("PopOnAbortMultiKey")
	{
		bool bHits[Count] = { false };
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
				AutoRTFM::PopOnAbortHandler(&bHits[Index]);
			}

			AutoRTFM::AbortTransaction();
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
	};

	BENCHMARK("PopOnAbortShortSingleKey")
	{
		bool bHit = false;
		unsigned AbortCounter = 0;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnAbort([&] { AbortCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnAbortHandler(&bHit);
			}

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bHit);
		REQUIRE(Count == AbortCounter);
	};

	BENCHMARK("PopOnAbortLongSingleKey")
	{
		bool bHit = false;
		unsigned AbortCounter = 0;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnAbort([&] { AbortCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnAbortHandler(&bHit);
			}

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bHit);
		REQUIRE(Count == AbortCounter);
	};

	BENCHMARK("PopOnAbortShortMultiKey")
	{
		bool bHits[Count] = { false };
		unsigned AbortCounter = 0;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnAbort([&] { AbortCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnAbortHandler(&bHits[Index]);
			}

			AutoRTFM::AbortTransaction();
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
		REQUIRE(Count == AbortCounter);
	};

	BENCHMARK("PopOnAbortLongMultiKey")
	{
		bool bHits[Count] = { false };
		unsigned AbortCounter = 0;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnAbort([&] { AbortCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnAbortHandler(&bHits[Index]);
			}

			AutoRTFM::AbortTransaction();
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
		REQUIRE(Count == AbortCounter);
	};
}

TEST_CASE("Benchmarks.PopAllOnAbortHandlers")
{
	constexpr unsigned Count = 16 * 128;

	BENCHMARK("Pop")
	{
		bool bHit = false;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnAbortHandler(&bHit);
			}

			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bHit);
	};

	BENCHMARK("PopAll")
	{
		bool bHit = false;
		AutoRTFM::Testing::Abort([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnAbortHandler(&bHit, [&] { bHit = true; });
			}

			AutoRTFM::PopAllOnAbortHandlers(&bHit);
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(!bHit);
	};
}

TEST_CASE("Benchmarks.PopOnCommitHandler")
{
	constexpr unsigned int Count = 16 * 128;

	BENCHMARK("PopOnCommitSingleKey")
	{
		bool bHit = false;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHit, [&] { bHit = true; });
				AutoRTFM::PopOnCommitHandler(&bHit);
			}
		});
		REQUIRE(!bHit);
	};

	BENCHMARK("PopOnCommitMultiKey")
	{
		bool bHits[Count] = { false };
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
				AutoRTFM::PopOnCommitHandler(&bHits[Index]);
			}
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
	};

	BENCHMARK("PopOnCommitShortSingleKey")
	{
		bool bHit = false;
		unsigned int CallbackCounter = 0;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnCommit([&] { CallbackCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnCommitHandler(&bHit);
			}
		});
		REQUIRE(!bHit);
		REQUIRE(Count == CallbackCounter);
	};

	BENCHMARK("PopOnCommitLongSingleKey")
	{
		bool bHit = false;
		unsigned int CallbackCounter = 0;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnCommit([&] { CallbackCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnCommitHandler(&bHit);
			}
		});
		REQUIRE(!bHit);
		REQUIRE(Count == CallbackCounter);
	};

	BENCHMARK("PopOnCommitShortMultiKey")
	{
		bool bHits[Count] = { false };
		unsigned int CallbackCounter = 0;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnCommit([&] { CallbackCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnCommitHandler(&bHits[Index]);
			}
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
		REQUIRE(Count == CallbackCounter);
	};

	BENCHMARK("PopOnCommitLongMultiKey")
	{
		bool bHits[Count] = { false };
		unsigned int CallbackCounter = 0;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHits[Index], [&bHits, Index] { bHits[Index] = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::OnCommit([&] { CallbackCounter++; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnCommitHandler(&bHits[Index]);
			}
		});

		for (unsigned int Index = 0; Index < Count; Index++)
		{
			REQUIRE(!bHits[Index]);
		}
		REQUIRE(Count == CallbackCounter);
	};
}

TEST_CASE("Benchmarks.PopAllOnCommitHandlers")
{
	constexpr unsigned int Count = 16 * 128;

	BENCHMARK("Pop")
	{
		bool bHit = false;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHit, [&] { bHit = true; });
			}

			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PopOnCommitHandler(&bHit);
			}
		});
		REQUIRE(!bHit);
	};

	BENCHMARK("PopAll")
	{
		bool bHit = false;
		AutoRTFM::Testing::Commit([&]
		{
			for (unsigned int Index = 0; Index < Count; Index++)
			{
				AutoRTFM::PushOnCommitHandler(&bHit, [&] { bHit = true; });
			}

			AutoRTFM::PopAllOnCommitHandlers(&bHit);
		});
		REQUIRE(!bHit);
	};
}

TEST_CASE("Benchmarks.TopLevelTransaction")
{
	constexpr int Iterations = 100;

	BENCHMARK("Commit")
	{
		int Counter = 0;
		for (int Index = 0; Index < Iterations; ++Index)
		{
			AutoRTFM::Testing::Commit([&]
			{
				++Counter;
			});
		}
		REQUIRE(Counter == Iterations);
	};

	BENCHMARK("Abort")
	{
		int Counter = 0;
		for (int Index = 0; Index < Iterations; ++Index)
		{
			AutoRTFM::Testing::Abort([&]
			{
				++Counter;
				AutoRTFM::AbortTransaction();
			});
		}
		REQUIRE(Counter == 0);
	};
}

namespace
{
	template <int Size>
	struct RawBytes 
	{
		std::array<uint8, Size> Data{};
		RawBytes<Size>& operator++()
		{
			for (uint8& N : Data)
			{
				++N;
			}
			return *this;
		}
	};
}

TEST_CASE("Benchmarks.Write")
{
	auto Benchmark = [&](Catch::Benchmark::Chronometer Meter, auto Value, int Stride)
	{
		using T = decltype(Value);
		constexpr size_t NumElements = 50000;
		std::vector<T> MyArray(NumElements * Stride);

		Meter.measure([&]
		{
			AutoRTFM::Transact([&]
			{
				T* Ptr = &MyArray.front();
				for (int Index = 0; Index < NumElements; ++Index)
				{
					*Ptr = Value;
					Ptr += Stride;
				}
			});
			++Value;
		});
	};

#define BENCH(T, Stride) \
	BENCHMARK_ADVANCED("Write: " #T " (Stride " #Stride ")")(Catch::Benchmark::Chronometer Meter) \
	{ \
		Benchmark(Meter, T{}, Stride); \
	};

	// Foldable
	BENCH(char, 1);
	BENCH(short, 1);
	BENCH(int32, 1);
	BENCH(int64, 1);
	BENCH(RawBytes<32>, 1);
	BENCH(RawBytes<100>, 1);

	// Non-foldable
	BENCH(char, 2);
	BENCH(short, 2);
	BENCH(int32, 2);
	BENCH(int64, 2);
	BENCH(RawBytes<32>, 2);
	BENCH(RawBytes<100>, 2);

#undef BENCH
}

// ---------------------------------------------------------------------------
// Cross-mode inlining benchmarks
//
// Measures runtime impact of calling functions across AutoRTFM mode
// boundaries. Each leaf function writes to heap memory (transaction-
// instrumented) and uses a hash accumulator to prevent constant folding
// and dead code elimination.
//
// Benchmark names use the call chain format: "Transact, Open, Enable"
// means a transaction calls an open function which calls an enable function.
// "(noinline)" marks functions forced noinline to measure call overhead.
// ---------------------------------------------------------------------------

namespace
{

constexpr unsigned BenchIterations = 1000000;

// Knuth multiplicative hash -- cheap, non-trivial transform that prevents
// the optimizer from constant-folding or strength-reducing the loop.
inline unsigned HashStep(unsigned Value)
{
	return Value * 2654435761u;
}

struct BenchmarkHelpers
{
	// Leaf functions -- one per mode, with inlined and noinline variants.

	AUTORTFM_OPEN static unsigned Open(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_OPEN [[clang::noinline]] static unsigned OpenNoInline(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_ENABLE static unsigned Enable(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_ENABLE [[clang::noinline]] static unsigned EnableNoInline(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_INTERNAL static unsigned Internal(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_INTERNAL [[clang::noinline]] static unsigned InternalNoInline(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_DISABLE static unsigned Disable(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	AUTORTFM_DISABLE [[clang::noinline]] static unsigned DisableNoInline(unsigned* Out, unsigned Value)
	{
		*Out = Value;
		return HashStep(Value);
	}

	// Composite functions -- call chains across mode boundaries.

	AUTORTFM_OPEN static unsigned OpenOpen(unsigned* Out, unsigned Value)
	{
		return Open(Out, Value);
	}

	AUTORTFM_OPEN static unsigned OpenEnable(unsigned* Out, unsigned Value)
	{
		return Enable(Out, Value);
	}

	AUTORTFM_OPEN static unsigned OpenDisable(unsigned* Out, unsigned Value)
	{
		return Disable(Out, Value);
	}

	AUTORTFM_OPEN [[clang::noinline]] static unsigned OpenDisableNoInline(unsigned* Out, unsigned Value)
	{
		return DisableNoInline(Out, Value);
	}
};

} // anonymous namespace

TEST_CASE("Benchmarks.CrossModeInlining")
{
	unsigned Sink = 0;
	auto* SinkPtr = &Sink;

#define BENCH_TRANSACT(Name, Func) \
	BENCHMARK(Name) \
	{ \
		AutoRTFM::Transact([&] \
		{ \
			unsigned Acc = 1; \
			for (unsigned I = 0; I < BenchIterations; ++I) \
			{ \
				Acc = BenchmarkHelpers::Func(SinkPtr, Acc); \
			} \
			*SinkPtr = Acc; \
		}); \
	}

#define BENCH_OPEN(Name, Func) \
	BENCHMARK(Name) \
	{ \
		unsigned Acc = 1; \
		for (unsigned I = 0; I < BenchIterations; ++I) \
		{ \
			Acc = BenchmarkHelpers::Func(SinkPtr, Acc); \
		} \
		*SinkPtr = Acc; \
	}

	// Baseline.
	BENCH_OPEN("Enable (baseline)",                    Enable);

	// Single mode outside a transaction.
	BENCH_OPEN("Open",                                 Open);
	BENCH_OPEN("Open (noinline)",                      OpenNoInline);
	BENCH_OPEN("Internal",                             Internal);
	BENCH_OPEN("Internal (noinline)",                  InternalNoInline);
	BENCH_OPEN("Disable",                              Disable);
	BENCH_OPEN("Disable (noinline)",                   DisableNoInline);

	// Single mode from a transaction.
	BENCH_TRANSACT("Transact, Open",                   Open);
	BENCH_TRANSACT("Transact, Open (noinline)",        OpenNoInline);
	BENCH_TRANSACT("Transact, Enable",                 Enable);
	BENCH_TRANSACT("Transact, Enable (noinline)",      EnableNoInline);
	BENCH_TRANSACT("Transact, Internal",               Internal);
	BENCH_TRANSACT("Transact, Internal (noinline)",    InternalNoInline);

	// Call chains from a transaction.
	BENCH_TRANSACT("Transact, Open, Open",             OpenOpen);
	BENCH_TRANSACT("Transact, Open, Enable",           OpenEnable);
	BENCH_TRANSACT("Transact, Open, Disable",          OpenDisable);
	BENCH_TRANSACT("Transact, Open, Disable (noinline)", OpenDisableNoInline);

#undef BENCH_TRANSACT
#undef BENCH_OPEN
}
