// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlockAllocator.h"
#include "Catch2Includes.h"
#include <cstddef>

TEST_CASE("BlockAllocator")
{
	static constexpr size_t AllocatorBlockSize = 128;
	static constexpr size_t AllocatorAlignment = 16;
	static constexpr size_t AllocatorGrowthPercentage = 100; // Fixed size
	AutoRTFM::TBlockAllocator<AllocatorBlockSize, AllocatorAlignment, AllocatorGrowthPercentage> BlockAllocator;

	struct FAllocationInfo
	{
		// The pointer returned by Allocate().
		std::byte* Ptr;
		// The allocation size in bytes.
		size_t Size;
		// The value written into the allocated memory.
		std::byte Value;
	};
	std::vector<FAllocationInfo> Allocations;

	// Helper that calls BlockAllocator.Allocate(Size, Alignment) and returns
	// the allocated pointer cast to std::byte*. The allocated memory is
	// memset() with a value, and the allocation pointer, size and memset()
	// value is recorded into Allocations so the memory can be checked at the
	// end of the test. The allocated pointer is checked for the requested
	// alignment.
	auto Allocate = [&](size_t Size, size_t Alignment)
	{
		std::byte* Ptr = static_cast<std::byte*>(BlockAllocator.Allocate(Size, Alignment));
		REQUIRE(reinterpret_cast<uintptr_t>(Ptr) % Alignment == 0);
		std::byte Value = static_cast<std::byte>(Allocations.size());
		memset(Ptr, static_cast<int>(Value), Size);
		Allocations.push_back(FAllocationInfo{Ptr, Size, Value});
		return static_cast<std::byte*>(Ptr);
	};

	SECTION("Contiguous")
	{
		size_t AllocationSize = AllocatorBlockSize / 8;
		size_t AllocationAlignment = 4;

		std::byte* FirstInBlock = Allocate(AllocationSize, AllocationAlignment);
		for (size_t BlockIndex = 0; BlockIndex < 4; BlockIndex++)
		{
			size_t Offset = AllocationSize;
			for (; Offset < AllocatorBlockSize; Offset += AllocationSize)
			{
				std::byte* Allocation = Allocate(AllocationSize, AllocationAlignment);
				REQUIRE(Allocation == FirstInBlock + Offset);
			}
			std::byte* NextBlock = Allocate(AllocationSize, AllocationAlignment);
			REQUIRE(NextBlock != FirstInBlock + Offset);
			FirstInBlock = NextBlock;
		}
	}

	SECTION("Size")
	{
		for (size_t I = 0; I < 1000; I++)
		{
			size_t Size = std::max<size_t>(1, (I * 31) % AllocatorBlockSize);
			Allocate(Size, /* Alignment */ 4);
		}
	}

	SECTION("Alignment")
	{
		for (size_t Alignment = 1; Alignment <= AllocatorAlignment; Alignment *= 2)
		{
			for (size_t I = 0; I < 100; I++)
			{
				Allocate(/* Size */ 4, Alignment);
			}
		}
	}

	SECTION("LargeAllocations")
	{
		SECTION("Large, Small, Small")
		{
			Allocate(AllocatorBlockSize*4, 4);
			Allocate(4, 4);
			Allocate(4, 4);
		}
		SECTION("Small, Large, Small")
		{
			Allocate(4, 4);
			Allocate(AllocatorBlockSize*4, 4);
			Allocate(4, 4);
		}
		SECTION("Small, Small, Large")
		{
			Allocate(4, 4);
			Allocate(4, 4);
			Allocate(AllocatorBlockSize*4, 4);
		}
	}

	// Check that the allocated memory is as expected.
	for (FAllocationInfo AllocationInfo : Allocations)
	{
		for (size_t Offset = 0; Offset < AllocationInfo.Size; Offset++)
		{
			REQUIRE(AllocationInfo.Ptr[Offset] == AllocationInfo.Value);
		}
	}
}
