// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "CoreGlobals.h"
#include "Logging/LogVerbosity.h"
#include "Misc/FeedbackContext.h"

#include <catch_amalgamated.hpp>
#include <charconv>
#include <cstring>
#include <system_error>

#ifdef __clang__
#define CLANG_BEGIN_DISABLE_OPTIMIZATIONS _Pragma("clang optimize off")
#define CLANG_END_DISABLE_OPTIMIZATIONS   _Pragma("clang optimize on")
#define CLANG_BEGIN_DISABLE_WARN_FORMAT_TRUNCATED \
	_Pragma("clang diagnostic push") \
	_Pragma("clang diagnostic ignored \"-Wunknown-warning-option\"") \
	_Pragma("clang diagnostic ignored \"-Wformat-truncation\"")
#define CLANG_END_DISABLE_WARN_FORMAT_TRUNCATED \
	_Pragma("clang diagnostic pop")
#else
#define CLANG_BEGIN_DISABLE_OPTIMIZATIONS
#define CLANG_END_DISABLE_OPTIMIZATIONS
#define CLANG_BEGIN_DISABLE_WARN_FORMAT_TRUNCATED
#define CLANG_END_DISABLE_WARN_FORMAT_TRUNCATED
#endif

#ifdef _MSC_VER
#define MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION \
		__pragma(warning(push)) \
		__pragma(warning(disable : 4996))
#define MSVC_END_DISABLE_UNSAFE_FN  __pragma(warning(pop))
#else
#define MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION
#define MSVC_END_DISABLE_UNSAFE_FN
#endif

namespace
{

// The expected AutoRTFM warning message when attempting to printf() using a
// format string containing a '%n' format specifier.
static constexpr char kPercentNWarning[] = "AutoRTFM does not support format strings containing '%n'";

// CheckWCharFunction calls Function with WChar outside a transaction, in an
// aborted transaction and committed transaction. The value returned when called
// in the committed transaction is expected to match the value returned when
// called outside the transaction.
// Function must be the deterministic function with the signature 'int(wchar_t)'.
template<typename FUNCTION>
void CheckWCharFunction(FUNCTION&& Function, wchar_t WChar)
{
	int Expect = Function(WChar);

	SECTION("With Abort")
	{
		int Got = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Got = Function(WChar);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Got);
	}

	SECTION("With Commit")
	{
		int Got = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Got = Function(WChar); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(Expect == Got);
	}
}

// CheckStringToNumberFunction calls Function with String outside a transaction,
// in an aborted transaction and committed transaction, once with a valid
// end pointer and again with a null end pointer. The value returned and the
// assigned end-pointer when called in the committed transaction is expected to
// match the values when called outside the transaction.
// Function must be the deterministic function with the signature
// 'RESULT(const CHAR* String, CHAR** EndPtr, ADDITIONAL_ARGS...)'.
template<typename RESULT, typename CHAR, typename ... ADDITIONAL_ARGS>
void CheckStringToNumberFunction(RESULT Function(const CHAR*, CHAR**, ADDITIONAL_ARGS...), const CHAR* String, ADDITIONAL_ARGS... AdditionalArgs)
{
	CHAR* ExpectEndPtr = nullptr;
	RESULT ExpectValue = Function(String, &ExpectEndPtr, AdditionalArgs...);

	CHAR* GotEndPtr = nullptr;
	RESULT GotValue = 0;

	SECTION("With EndPtr")
	{
		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				GotValue = Function(String, &GotEndPtr, AdditionalArgs...);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == GotValue);
			REQUIRE(nullptr == GotEndPtr);
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				GotValue = Function(String, &GotEndPtr, AdditionalArgs...);
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(ExpectValue == GotValue);
			REQUIRE(ExpectEndPtr == GotEndPtr);
		}
	}
	SECTION("Null EndPtr")
	{
		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				GotValue = Function(String, nullptr, AdditionalArgs...);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == GotValue);
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { GotValue = Function(String, nullptr, AdditionalArgs...); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(ExpectValue == GotValue);
		}
	}
}

// CheckStdToChars calls std::to_chars with (char*, char*, ARGS...) outside a
// transaction, in an aborted transaction and committed transaction.
// The value returned and the string buffer content is expected to match the
// values when called outside the transaction.
template<typename ... ARGS>
void CheckStdToChars(ARGS ... Args)
{
	static constexpr size_t BufferOffset = 2;
	static constexpr size_t BufferSize = 32;
	char ExpectedBuffer[]
	{
		'#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#',
		'#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '\0',
	};
	static_assert(sizeof(ExpectedBuffer) == BufferSize);
	const std::to_chars_result ExpectedResult = std::to_chars(&ExpectedBuffer[BufferOffset], &ExpectedBuffer[BufferSize-1], Args...);

	char Buffer[]
	{
		'#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#',
		'#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '#', '\0',
	};
	static_assert(sizeof(Buffer) == BufferSize);

	SECTION("With Abort")
	{
		AutoRTFM::Testing::Abort([&]
		{
			std::to_chars_result Result = std::to_chars(&Buffer[BufferOffset], &Buffer[BufferSize-1], Args...);
			REQUIRE(Result.ec == ExpectedResult.ec);
			REQUIRE(Result.ptr - Buffer == ExpectedResult.ptr - ExpectedBuffer);
			REQUIRE(std::string_view(Buffer) == std::string_view(ExpectedBuffer));
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(std::string_view(Buffer) == "###############################");
	}

	SECTION("With Commit")
	{
		AutoRTFM::Testing::Commit([&]
		{
			std::to_chars_result Result = std::to_chars(&Buffer[BufferOffset], &Buffer[BufferSize-1], Args...);
			REQUIRE(Result.ec == ExpectedResult.ec);
			REQUIRE(Result.ptr - Buffer == ExpectedResult.ptr - ExpectedBuffer);
			REQUIRE(std::string_view(Buffer) == std::string_view(ExpectedBuffer));
		});
		REQUIRE(std::string_view(Buffer) == std::string_view(ExpectedBuffer));
	}
}

}  // anonymous namespace

TEST_CASE("CString.memcpy")
{
	const char* const From = "Kittie says meow";
	char To[] = "Doggie says woof";

	SECTION("Direct")
	{
		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				memcpy(To, From, 6);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Doggie says woof" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { memcpy(To, From, 6); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Kittie says woof" == std::string_view(To));
		}
	}

	SECTION("Indirect")
	{
		using FnPtr = void*(*)(void*, const void*, size_t);
		FnPtr Fn = &memcpy;

		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Fn(To, From, 6);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Doggie says woof" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Fn(To, From, 6); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Kittie says woof" == std::string_view(To));
		}
	}
}

TEST_CASE("CString.memmove")
{
	char To[] = "Hello, world!";

	SECTION("Direct")
	{
		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				memmove(To + 7, To, 5);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Hello, world!" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { memmove(To + 7, To, 5); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Hello, Hello!" == std::string_view(To));
		}
	}

	SECTION("Indirect")
	{
		using FnPtr = void*(*)(void*, const void*, size_t);
		FnPtr Fn = &memmove;

		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Fn(To + 7, To, 5);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Hello, world!" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Fn(To + 7, To, 5); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Hello, Hello!" == std::string_view(To));
		}
	}
}

TEST_CASE("CString.memset")
{
	char To[] = "Hello, world!";

	SECTION("Direct")
	{
		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				memset(To + 7, 'X', 5);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Hello, world!" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { memset(To + 7, 'X', 5); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Hello, XXXXX!" == std::string_view(To));
		}
	}

	SECTION("Indirect")
	{
		using FnPtr = void*(*)(void*, int, size_t);
		FnPtr Fn = &memset;

		SECTION("With Abort")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Fn(To + 7, 'X', 5);
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE("Hello, world!" == std::string_view(To));
		}

		SECTION("With Commit")
		{
			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Fn(To + 7, 'X', 5); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE("Hello, XXXXX!" == std::string_view(To));
		}
	}
}

TEST_CASE("CString.strcpy")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	const char From[] = "Kittie says meow";
	char To[] = "Doggie says woof____";
	static_assert(sizeof(From) <= sizeof(To));

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			strcpy(To, From);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE("Doggie says woof____" == std::string_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { strcpy(To, From); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE("Kittie says meow" == std::string_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.strncpy")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	const char* const From = "Kittie says meow";
	char To[] = "Doggie says woof";

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			strncpy(To, From, 6);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE("Doggie says woof" == std::string_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { strncpy(To, From, 6); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE("Kittie says woof" == std::string_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.strcat")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	constexpr unsigned Size = 128;
	char To[Size] = "Hello";

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			strcat(To, ", world!");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE("Hello" == std::string_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { strcat(To, ", world!"); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE("Hello, world!" == std::string_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.strncat")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	constexpr unsigned Size = 128;
	char To[Size] = "Hello";

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			strncat(To, ", world! Not this!", 8);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE("Hello" == std::string_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { strncat(To, ", world! Not this!", 8); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE("Hello, world!" == std::string_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.memcmp")
{
	constexpr unsigned Size = 128;
	char A[Size] = "This";

	SECTION("With Abort")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Compare = memcmp(A, "That", 4);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Compare);
	}

	SECTION("With Commit")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Compare = memcmp(A, "That", 4); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(0 < Compare);
	}
}

TEST_CASE("CString.strcmp")
{
	const char* A = "This";

	SECTION("With Abort")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Compare = strcmp(A, "That");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Compare);
	}

	SECTION("With Commit")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Compare = strcmp(A, "That"); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(0 < Compare);
	}
}

TEST_CASE("CString.strncmp")
{
	const char* A = "This";

	SECTION("With Abort")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Compare = strncmp(A, "That", 3);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Compare);
	}

	SECTION("With Commit")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Compare = strncmp(A, "That", 3); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(0 < Compare);
	}
}

TEST_CASE("CString.strchr")
{
	const char* A = "Thinking";

	SECTION("With Abort")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Value = strchr(A, 'i');
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Value);
	}

	SECTION("With Commit")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Value = strchr(A, 'i'); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE((A + 2) == Value);
	}
}

TEST_CASE("CString.strrchr")
{
	const char* A = "Thinking";

	SECTION("With Abort")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Value = strrchr(A, 'i');
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Value);
	}

	SECTION("With Commit")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Value = strrchr(A, 'i'); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE((A + 5) == Value);
	}
}

TEST_CASE("CString.strstr")
{
	const char* A = "This";

	SECTION("With Abort")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Value = strstr(A, "is");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Value);
	}

	SECTION("With Commit")
	{
		const char* Value = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Value = strstr(A, "is"); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE((A + 2) == Value);
	}
}

TEST_CASE("CString.strlen")
{
	const char* A = "This";

	SECTION("With Abort")
	{
		size_t Value = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Value = strlen(A);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Value);
	}

	SECTION("With Commit")
	{
		size_t Value = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Value = strlen(A); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(4 == Value);
	}
}

TEST_CASE("CString.wcscpy")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	const wchar_t From[] = L"Kittie says meow";
	wchar_t To[] = L"Doggie says woof____";
	static_assert(sizeof(From) <= sizeof(To));

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			wcscpy(To, From);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(std::wstring_view(L"Doggie says woof____") == std::wstring_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { wcscpy(To, From); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(std::wstring_view(L"Kittie says meow") == std::wstring_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.wcsncpy")
{
	MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION

	const wchar_t* const From = L"Kittie says meow";
	wchar_t To[] = L"Doggie says woof";

	SECTION("With Abort")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			wcsncpy(To, From, 6);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(std::wstring_view(L"Doggie says woof") == std::wstring_view(To));
	}

	SECTION("With Commit")
	{
		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { wcsncpy(To, From, 6); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(std::wstring_view(L"Kittie says woof") == std::wstring_view(To));
	}

	MSVC_END_DISABLE_UNSAFE_FN
}

TEST_CASE("CString.wcsncmp")
{
	const wchar_t* A = L"This";

	SECTION("With Abort")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Compare = wcsncmp(A, L"That", 3);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Compare);
	}

	SECTION("With Commit")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Compare = wcsncmp(A, L"That", 3); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(0 < Compare);
	}
}

TEST_CASE("CString.wcschr")
{
	SECTION("Const")
	{
		const wchar_t* A = L"Thinking";

		SECTION("With Abort")
		{
			const wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Value = wcschr(A, L'i');
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(nullptr == Value);
		}

		SECTION("With Commit")
		{
			const wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Value = wcschr(A, L'i'); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE((A + 2) == Value);
		}
	}

	SECTION("Non-Const")
	{
		wchar_t A[] = L"Thinking";

		SECTION("With Abort")
		{
			wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Value = wcschr(A, L'i');
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(nullptr == Value);
		}

		SECTION("With Commit")
		{
			wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Value = wcschr(A, L'i'); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE((A + 2) == Value);
		}
	}
}

TEST_CASE("CString.wcsstr")
{
	SECTION("Const")
	{
		const wchar_t* A = L"Thinking";

		SECTION("With Abort")
		{
			const wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Value = wcsstr(A, L"ink");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(nullptr == Value);
		}

		SECTION("With Commit")
		{
			const wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Value = wcsstr(A, L"ink"); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE((A + 2) == Value);
		}
	}

	SECTION("Non-Const")
	{
		wchar_t A[] = L"Thinking";

		SECTION("With Abort")
		{
			wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Value = wcsstr(A, L"ink");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(nullptr == Value);
		}

		SECTION("With Commit")
		{
			wchar_t* Value = nullptr;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&] { Value = wcsstr(A, L"ink"); });
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE((A + 2) == Value);
		}
	}
}

TEST_CASE("CString.wcsrchr")
{
	const wchar_t* A = L"This";
	SECTION("With Abort")
	{
		const wchar_t* Return = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Return = wcsrchr(A, 'h');
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(nullptr == Return);
	}

	SECTION("With Commit")
	{
		const wchar_t* Return = nullptr;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Return = wcsrchr(A, 'h'); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(nullptr != Return);
		REQUIRE('h' == *Return);
	}
}

TEST_CASE("CString.wcscmp")
{
	const wchar_t* A = L"This";

	SECTION("With Abort")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Compare = wcscmp(A, L"That");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Compare);
	}

	SECTION("With Commit")
	{
		int Compare = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Compare = wcscmp(A, L"That"); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(0 < Compare);
	}
}

TEST_CASE("CString.wcslen")
{
	const wchar_t* A = L"This";

	SECTION("With Abort")
	{
		size_t Value = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Value = wcslen(A);
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Value);
	}

	SECTION("With Commit")
	{
		size_t Value = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&] { Value = wcslen(A); });
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(4 == Value);
	}
}

TEST_CASE("CString.wcstod")
{
	CheckStringToNumberFunction(wcstod, L"42.5abc");
}

TEST_CASE("CString.wcstof")
{
	CheckStringToNumberFunction(wcstof, L"42.5abc");
}

TEST_CASE("CString.wcstold")
{
	CheckStringToNumberFunction(wcstold, L"42.5abc");
}

TEST_CASE("CString.strtol")
{
	CheckStringToNumberFunction(strtol, "42.5abc", 10);
}

TEST_CASE("CString.strtoll")
{
	CheckStringToNumberFunction(strtoll, "42.5abc", 10);
}

TEST_CASE("CString.strtoul")
{
	CheckStringToNumberFunction(strtoul, "42.5abc", 10);
}

TEST_CASE("CString.strtoull")
{
	CheckStringToNumberFunction(strtoull, "42.5abc", 10);
}

TEST_CASE("CString.wcstol")
{
	CheckStringToNumberFunction(wcstol, L"42abc", 10);
}

TEST_CASE("CString.wcstoll")
{
	CheckStringToNumberFunction(wcstoll, L"42abc", 10);
}

TEST_CASE("CString.wcstoul")
{
	CheckStringToNumberFunction(wcstoul, L"42abc", 10);
}

TEST_CASE("CString.wcstoull")
{
	CheckStringToNumberFunction(wcstoull, L"42abc", 10);
}

TEST_CASE("std::to_chars")
{
	CheckStdToChars(/* value */ static_cast<int>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<unsigned int>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<int8_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<uint8_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<int16_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<uint16_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<int32_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<uint32_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<int64_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<uint64_t>(42), /* base */ 8);
	CheckStdToChars(/* value */ static_cast<float>(42.24f));
	CheckStdToChars(/* value */ static_cast<float>(42.24f), /* fmt */ std::chars_format::hex);
	CheckStdToChars(/* value */ static_cast<float>(42.24f), /* fmt */ std::chars_format::hex, /* precision */ 3);
	CheckStdToChars(/* value */ static_cast<double>(42.24));
	CheckStdToChars(/* value */ static_cast<double>(42.24), /* fmt */ std::chars_format::hex);
	CheckStdToChars(/* value */ static_cast<double>(42.24), /* fmt */ std::chars_format::hex, /* precision */ 3);
	CheckStdToChars(/* value */ static_cast<long double>(42.24));
	CheckStdToChars(/* value */ static_cast<long double>(42.24), /* fmt */ std::chars_format::hex);
	CheckStdToChars(/* value */ static_cast<long double>(42.24), /* fmt */ std::chars_format::hex, /* precision */ 3);
}

#if PLATFORM_WINDOWS
TEST_CASE("CString._wcstoi64")
{
	CheckStringToNumberFunction(_wcstoi64, L"42abc", 10);
}

TEST_CASE("CString._wcstoui64")
{
	CheckStringToNumberFunction(_wcstoui64, L"42abc", 10);
}

TEST_CASE("CString._wcstod_l")
{
	CheckStringToNumberFunction(_wcstod_l, L"42.5abc", _get_current_locale());
}

TEST_CASE("CString._wcstof_l")
{
	CheckStringToNumberFunction(_wcstof_l, L"42.5abc", _get_current_locale());
}

TEST_CASE("CString._wcstold_l")
{
	CheckStringToNumberFunction(_wcstold_l, L"42.5abc", _get_current_locale());
}

TEST_CASE("CString._wcstol_l")
{
	CheckStringToNumberFunction(_wcstol_l, L"42abc", 10, _get_current_locale());
}

TEST_CASE("CString._wcstoll_l")
{
	CheckStringToNumberFunction(_wcstoll_l, L"42abc", 10, _get_current_locale());
}

TEST_CASE("CString._wcstoul_l")
{
	CheckStringToNumberFunction(_wcstoul_l, L"42abc", 10, _get_current_locale());
}

TEST_CASE("CString._wcstoull_l")
{
	CheckStringToNumberFunction(_wcstoull_l, L"42abc", 10, _get_current_locale());
}

TEST_CASE("CString._wcstoi64_l")
{
	CheckStringToNumberFunction(_wcstoi64_l, L"42abc", 10, _get_current_locale());
}

TEST_CASE("CString._wcstoui64_l")
{
	CheckStringToNumberFunction(_wcstoui64_l, L"42abc", 10, _get_current_locale());
}

#endif

TEST_CASE("CString.iswupper")
{
	CheckWCharFunction(iswupper, L'X');
	CheckWCharFunction(iswupper, L'x');
}

TEST_CASE("CString.iswlower")
{
	CheckWCharFunction(iswlower, L'X');
	CheckWCharFunction(iswlower, L'x');
}

TEST_CASE("CString.iswalpha")
{
	CheckWCharFunction(iswalpha, L'X');
	CheckWCharFunction(iswalpha, L'5');
}

TEST_CASE("CString.iswgraph")
{
	CheckWCharFunction(iswgraph, L'X');
	CheckWCharFunction(iswgraph, L'x');
}

TEST_CASE("CString.iswprint")
{
	CheckWCharFunction(iswprint, L'X');
	CheckWCharFunction(iswprint, L'x');
}

TEST_CASE("CString.iswpunct")
{
	CheckWCharFunction(iswpunct, L'X');
	CheckWCharFunction(iswpunct, L'^');
}

TEST_CASE("CString.iswalnum")
{
	CheckWCharFunction(iswalnum, L'X');
	CheckWCharFunction(iswalnum, L'6');
	CheckWCharFunction(iswalnum, L' ');
}

TEST_CASE("CString.iswdigit")
{
	CheckWCharFunction(iswdigit, L'X');
	CheckWCharFunction(iswdigit, L'5');
}

TEST_CASE("CString.iswxdigit")
{
	CheckWCharFunction(iswxdigit, L'X');
	CheckWCharFunction(iswxdigit, L'q');
}

TEST_CASE("CString.iswspace")
{
	CheckWCharFunction(iswspace, L'X');
	CheckWCharFunction(iswspace, L' ');
}

TEST_CASE("CString.iswcntrl")
{
	CheckWCharFunction(iswcntrl, L'X');
	CheckWCharFunction(iswcntrl, L'\u2028');
}

TEST_CASE("CString.snprintf")
{
	static constexpr size_t BufferSize = 64;
	char Buffer[BufferSize] = "_____________________";

	SECTION("Fits in buffer")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = snprintf(Buffer, BufferSize, "cat says '%s'!", "meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE("_____________________" == std::string_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = snprintf(Buffer, BufferSize, "cat says '%s'!", "meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(16 == Count);
			REQUIRE("cat says 'meow'!" == std::string_view(Buffer));
		}
	}

	SECTION("Buffer too small")
	{
		CLANG_BEGIN_DISABLE_WARN_FORMAT_TRUNCATED

		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = snprintf(Buffer, 8, "cat says '%s'!", "meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE("_____________________" == std::string_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = snprintf(Buffer, 8, "cat says '%s'!", "meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(16 == Count);
			REQUIRE("cat say" == std::string_view(Buffer));
		}

		CLANG_END_DISABLE_WARN_FORMAT_TRUNCATED
	}

	SECTION("Null buffer")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = snprintf(nullptr, 0, "cat says '%s'!", "meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = snprintf(nullptr, 0, "cat says '%s'!", "meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(16 == Count);
		}
	}

	SECTION("PercentN")
	{
		AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			snprintf(Buffer, 8, "%n", &Count);
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
		REQUIRE(0 == Count);
		REQUIRE("_____________________" == std::string_view(Buffer));
		REQUIRE(WarningContext.HasWarning(kPercentNWarning));
	}

	SECTION("PercentPercentN")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = snprintf(Buffer, 8, "%%n");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE("_____________________" == std::string_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = snprintf(Buffer, 8, "%%n");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(2 == Count);
			REQUIRE("%n" == std::string_view(Buffer));
		}
	}
}

TEST_CASE("CString.swprintf")
{
	static constexpr size_t BufferSize = 64;
	wchar_t Buffer[BufferSize] = L"_____________________";

	SECTION("Fits in buffer")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = swprintf(Buffer, BufferSize, L"cat says '%ls'!", L"meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE(std::wstring_view(L"_____________________") == std::wstring_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = swprintf(Buffer, BufferSize, L"cat says '%ls'!", L"meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(16 == Count);
			REQUIRE(std::wstring_view(L"cat says 'meow'!") == std::wstring_view(Buffer));
		}
	}

	SECTION("Buffer too small")
	{
		CLANG_BEGIN_DISABLE_WARN_FORMAT_TRUNCATED

		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = swprintf(Buffer, 8, L"cat says '%ls'!", L"meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE(std::wstring_view(L"_____________________") == std::wstring_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = swprintf(Buffer, 8, L"cat says '%ls'!", L"meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(0 > Count);
			REQUIRE(std::wstring_view(L"cat say") == std::wstring_view(Buffer, 7));
		}

		CLANG_END_DISABLE_WARN_FORMAT_TRUNCATED
	}

	SECTION("Null buffer")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = swprintf(nullptr, 0, L"cat says '%ls'!", L"meow");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = swprintf(nullptr, 0, L"cat says '%ls'!", L"meow");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(((0 > Count) || (16 == Count)));
		}
	}

	SECTION("PercentN")
	{
		AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			swprintf(Buffer, 8, L"%n", &Count);
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
		REQUIRE(0 == Count);
		REQUIRE(std::wstring_view(L"_____________________") == std::wstring_view(Buffer));
		REQUIRE(WarningContext.HasWarning(kPercentNWarning));
	}

	SECTION("PercentPercentN")
	{
		SECTION("With Abort")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Count = swprintf(Buffer, 8, L"%%n");
				AutoRTFM::AbortTransaction();
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(0 == Count);
			REQUIRE(std::wstring_view(L"_____________________") == std::wstring_view(Buffer));
		}

		SECTION("With Commit")
		{
			int Count = 0;

			CLANG_BEGIN_DISABLE_OPTIMIZATIONS
			AutoRTFM::Testing::Commit([&]
			{
				Count = swprintf(Buffer, 8, L"%%n");
			});
			CLANG_END_DISABLE_OPTIMIZATIONS

			REQUIRE(2 == Count);
			REQUIRE(std::wstring_view(L"%n") == std::wstring_view(Buffer));
		}
	}
}

TEST_CASE("CString.printf")
{
	SECTION("With Abort")
	{
		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Count = printf("AutoRTFM printf test: %%n\n");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Count);
	}

	SECTION("With Commit")
	{
		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::Testing::Commit([&]
		{
			Count = printf("AutoRTFM printf test: %%n\n");
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(25 == Count);
	}

	SECTION("PercentN")
	{
		AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			printf("%n", &Count);
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
		REQUIRE(0 == Count);
		REQUIRE(WarningContext.HasWarning(kPercentNWarning));
	}
}

TEST_CASE("CString.wprintf")
{
#if PLATFORM_WINDOWS // wprintf() can error on linux
	SECTION("With Abort")
	{
		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Count = wprintf(L"AutoRTFM wprintf test: %%n\n");
			AutoRTFM::AbortTransaction();
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == Count);
	}

	SECTION("With Commit")
	{
		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Count = wprintf(L"AutoRTFM wprintf test: %%n\n");
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
		REQUIRE(26 == Count);
	}
#endif // PLATFORM_WINDOWS

	SECTION("PercentN")
	{
		AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

		int Count = 0;

		CLANG_BEGIN_DISABLE_OPTIMIZATIONS
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			wprintf(L"%n", &Count);
		});
		CLANG_END_DISABLE_OPTIMIZATIONS

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByLanguage == Result);
		REQUIRE(0 == Count);
		REQUIRE(WarningContext.HasWarning(kPercentNWarning));
	}
}

#undef CLANG_BEGIN_DISABLE_OPTIMIZATIONS
#undef CLANG_END_DISABLE_OPTIMIZATIONS
#undef MSVC_BEGIN_DISABLE_WARN_UNSAFE_FUNCTION
#undef MSVC_END_DISABLE_UNSAFE_FN
