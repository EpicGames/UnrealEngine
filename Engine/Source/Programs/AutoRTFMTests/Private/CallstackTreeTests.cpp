// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallstackTree.h"
#include "Catch2Includes.h"

TEST_CASE("CallstackTree")
{
	using FCallstackTree = AutoRTFM::TCallstackTree<>;

	using FHandle = FCallstackTree::FHandle;
	static constexpr FCallstackTree::FHandle InvalidHandle = FCallstackTree::InvalidHandle;

	SECTION("Empty")
	{
		FCallstackTree Tree;
		REQUIRE(Tree.Add(0, nullptr) == InvalidHandle);
		REQUIRE((Tree.Get(InvalidHandle).IsEmpty()));
	}

	SECTION("Shallow, Unique")
	{
		FCallstackTree Tree;
		AutoRTFM::TStack<autortfm_callstack_frame, 1> A{1};
		FHandle HandleA = Tree.Add(A.Num(), &A.Front());
		REQUIRE(HandleA == 1);

		AutoRTFM::TStack<autortfm_callstack_frame, 1> B{2};
		FHandle HandleB = Tree.Add(B.Num(), &B.Front());
		REQUIRE(HandleB == 2);

		AutoRTFM::TStack<autortfm_callstack_frame, 1> C{3};
		FHandle HandleC = Tree.Add(C.Num(), &C.Front());
		REQUIRE(HandleC == 3);

		REQUIRE((Tree.Get(HandleA) == A));
		REQUIRE((Tree.Get(HandleB) == B));
		REQUIRE((Tree.Get(HandleC) == C));
	}

	SECTION("Shallow, Duplicates")
	{
		FCallstackTree Tree;
		AutoRTFM::TStack<autortfm_callstack_frame, 1> A{1};
		FHandle HandleA = Tree.Add(A.Num(), &A.Front());
		REQUIRE(HandleA == 1);

		AutoRTFM::TStack<autortfm_callstack_frame, 1> B{2};
		FHandle HandleB = Tree.Add(B.Num(), &B.Front());
		REQUIRE(HandleB == 2);

		AutoRTFM::TStack<autortfm_callstack_frame, 1> C{1};
		FHandle HandleC = Tree.Add(C.Num(), &C.Front());
		REQUIRE(HandleC == 1);

		REQUIRE((Tree.Get(HandleA) == A));
		REQUIRE((Tree.Get(HandleB) == B));
		REQUIRE((Tree.Get(HandleC) == C));
	}
	
	SECTION("Deep")
	{
		FCallstackTree Tree;
		AutoRTFM::TStack<autortfm_callstack_frame, 5> A{5, 4, 3, 2, 1};
		FHandle HandleA = Tree.Add(A.Num(), &A.Front());
		REQUIRE(HandleA == 5);

		AutoRTFM::TStack<autortfm_callstack_frame, 5> B{5, 4, 3, 2, 1};
		FHandle HandleB = Tree.Add(B.Num(), &B.Front());
		REQUIRE(HandleB == 5);

		AutoRTFM::TStack<autortfm_callstack_frame, 5> C{3, 2, 1};
		FHandle HandleC = Tree.Add(C.Num(), &C.Front());
		REQUIRE(HandleC == 3);

		AutoRTFM::TStack<autortfm_callstack_frame, 5> D{5, 4};
		FHandle HandleD = Tree.Add(D.Num(), &D.Front());
		REQUIRE(HandleD == 7);

		AutoRTFM::TStack<autortfm_callstack_frame, 5> E{2, 1};
		FHandle HandleE = Tree.Add(E.Num(), &E.Front());
		REQUIRE(HandleE == 2);

		REQUIRE((Tree.Get(HandleA) == A));
		REQUIRE((Tree.Get(HandleB) == B));
		REQUIRE((Tree.Get(HandleC) == C));
		REQUIRE((Tree.Get(HandleD) == D));
		REQUIRE((Tree.Get(HandleE) == E));
	}
}
