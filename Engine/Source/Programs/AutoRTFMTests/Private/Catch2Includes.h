// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// we need to include the MainCPP include *before* we include catch_amalgamated or we get issues with windows symbols
#include "CoreMinimal.h"
#include "LaunchEngineLoop.h"
#include "AutoRTFM.h"

#if defined(REQUIRE) || defined(CATCH_REQUIRE)
#error "Catch2 already included"
#endif

// Prefixes all the 'real' Catch2 macros with 'CATCH_'.
// We call these with the AutoRTFM-safe, unprefixed forms below.
#define CATCH_CONFIG_PREFIX_ALL 1


namespace Catch::Benchmark
{
    template <typename T>
    AUTORTFM_DISABLE inline void deoptimize_value(T&& x);

    template <typename Fn, typename... Args>
    AUTORTFM_DISABLE inline auto invoke_deoptimized(Fn&& fn, Args&&... args) -> std::enable_if_t<!std::is_same<void, decltype(fn(args...))>::value>;

    template <typename Fn, typename... Args>
    AUTORTFM_DISABLE inline auto invoke_deoptimized(Fn&& fn, Args&&... args) -> std::enable_if_t<std::is_same<void, decltype(fn(args...))>::value>;

    struct AUTORTFM_DISABLE Benchmark;
    struct AUTORTFM_DISABLE Chronometer;

    namespace Detail { struct AUTORTFM_DISABLE BenchmarkFunction; }
}

THIRD_PARTY_INCLUDES_START
#include <catch_amalgamated.hpp>
THIRD_PARTY_INCLUDES_END

////////////////////////////////////////////////////////////////////////////////
// AutoRTFM-safe versions of the Catch2 macros
////////////////////////////////////////////////////////////////////////////////

// Evaluates the expression (varargs) in the current transaction state, then
// switches to the open to call into the Catch2 assertion handler logic.
#define AUTORTFM_CATCH2_INTERNAL_CATCH_TEST(macroName, resultDisposition, ...) \
    do { /* NOLINT(bugprone-infinite-loop) */ \
        auto Expr = Catch::Decomposer() <= __VA_ARGS__; \
        AutoRTFM::Open([&] \
        { \
            /* The expression should not be evaluated, but warnings should hopefully be checked */ \
            CATCH_INTERNAL_IGNORE_BUT_WARN(__VA_ARGS__); \
            Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, CATCH_INTERNAL_STRINGIFY(__VA_ARGS__), resultDisposition ); \
            INTERNAL_CATCH_TRY { \
                CATCH_INTERNAL_START_WARNINGS_SUPPRESSION \
                CATCH_INTERNAL_SUPPRESS_PARENTHESES_WARNINGS \
                catchAssertionHandler.handleExpr( Expr ); \
                CATCH_INTERNAL_STOP_WARNINGS_SUPPRESSION \
            } INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
            catchAssertionHandler.complete(); \
        }); \
    } while( (void)0, (false) && static_cast<const bool&>( !!(__VA_ARGS__) ) )

// Evaluates the matcher + arg in the current transaction state, then
// switches to the open to call into the Catch2 assertion handler logic.
#define AUTORTFM_CATCH2_INTERNAL_CHECK_THAT( macroName, matcher, resultDisposition, arg ) \
    do { \
        auto Expr = Catch::makeMatchExpr( arg, matcher ); \
        AutoRTFM::Open([&] \
        { \
            Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, CATCH_INTERNAL_STRINGIFY(arg) ", " CATCH_INTERNAL_STRINGIFY(matcher), resultDisposition ); \
            INTERNAL_CATCH_TRY { \
                catchAssertionHandler.handleExpr( Expr ); \
            } INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
            catchAssertionHandler.complete(); \
        }); \
    } while(false)

// Same as Catch2's INTERNAL_CATCH_SECTION, but the unique variable name is
// passed in, and can be used in a closed transaction.
#define AUTORTFM_CATCH2_SECTION(VAR_NAME, ...)                                           \
    CATCH_INTERNAL_START_WARNINGS_SUPPRESSION                                            \
    CATCH_INTERNAL_SUPPRESS_UNUSED_VARIABLE_WARNINGS                                     \
    if (auto VAR_NAME = AutoRTFMCatch2::Section( CATCH_INTERNAL_LINEINFO, __VA_ARGS__ )) \
    CATCH_INTERNAL_STOP_WARNINGS_SUPPRESSION


#ifdef CATCH_CONFIG_EXPERIMENTAL_STATIC_ANALYSIS_SUPPORT
    #define AUTORTFM_CATCH_TEST_CASE(TEST_NAME, ...) \
        AUTORTFM_DISABLE static void TEST_NAME(); \
        INTERNAL_CATCH_TESTCASE2(TEST_NAME)
#else  // ^^ CATCH_CONFIG_EXPERIMENTAL_STATIC_ANALYSIS_SUPPORT | vv !CATCH_CONFIG_EXPERIMENTAL_STATIC_ANALYSIS_SUPPORT
    #define AUTORTFM_CATCH_TEST_CASE(TEST_NAME, ...) \
        AUTORTFM_DISABLE static void TEST_NAME(); \
        INTERNAL_CATCH_TESTCASE2(Catch::DefaultGroup, TEST_NAME, __VA_ARGS__)
#endif  // ^^ !CATCH_CONFIG_EXPERIMENTAL_STATIC_ANALYSIS_SUPPORT

#define REQUIRE( ... ) AUTORTFM_CATCH2_INTERNAL_CATCH_TEST( "REQUIRE", Catch::ResultDisposition::Normal, __VA_ARGS__ )
#define REQUIRE_THAT( arg, matcher ) AUTORTFM_CATCH2_INTERNAL_CHECK_THAT( "REQUIRE_THAT", matcher, Catch::ResultDisposition::Normal, arg )
#define FAIL(...) AutoRTFM::Open([&] { CATCH_FAIL(__VA_ARGS__); })
#define TEST_CASE(...) AUTORTFM_CATCH_TEST_CASE(INTERNAL_CATCH_UNIQUE_NAME(CATCH2_INTERNAL_TEST_), __VA_ARGS__)
#define TEMPLATE_TEST_CASE(...) CATCH_TEMPLATE_TEST_CASE(__VA_ARGS__)
#define BENCHMARK(...) CATCH_BENCHMARK(__VA_ARGS__)
#define BENCHMARK_ADVANCED(...) CATCH_BENCHMARK_ADVANCED(__VA_ARGS__)
#define SECTION(...) AUTORTFM_CATCH2_SECTION(INTERNAL_CATCH_UNIQUE_NAME(catch_internal_Section), __VA_ARGS__)

namespace AutoRTFMCatch2
{

// An AutoRTFM always-open wrapper around Catch::Section
class Section {
public:
    UE_AUTORTFM_ALWAYS_OPEN
    Section(Catch::SectionInfo&& Info) : Inner(std::move(Info)) {}

    UE_AUTORTFM_ALWAYS_OPEN
    Section(Catch::SourceLineInfo const& LineInfo, Catch::StringRef Name, const char* const = nullptr) : Inner(LineInfo, Name) {}

    UE_AUTORTFM_ALWAYS_OPEN
    ~Section() = default;

    UE_AUTORTFM_ALWAYS_OPEN
    explicit operator bool() const { return Inner.operator bool(); }

private:
    Catch::Section Inner;
};

} // AutoRTFMCatch2
