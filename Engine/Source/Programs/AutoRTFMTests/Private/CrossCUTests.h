// Copyright Epic Games, Inc. All Rights Reserved.

namespace CrossCU
{

int SomeFunction(int X);

struct FLargeStruct
{
	int Ints[32];

	static int Sum(FLargeStruct Struct);
};

} // CrossCU
