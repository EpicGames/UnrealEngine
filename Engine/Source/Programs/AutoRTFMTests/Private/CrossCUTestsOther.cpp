// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h"
#include "CrossCUTests.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace CrossCU
{
    
int SomeFunction(int X) { return X + 42; }

int FLargeStruct::Sum(FLargeStruct Struct)
{
    int Result = 0;
    for (int I : Struct.Ints)
    {
        Result += I;
    }
    return Result;
}

} // CrossCU

UE_ENABLE_OPTIMIZATION_SHIP
