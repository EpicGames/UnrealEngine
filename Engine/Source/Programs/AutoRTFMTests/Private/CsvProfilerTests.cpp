// Copyright Epic Games, Inc. All Rights Reserved.

#include "API.h"
#include "Async/TaskGraphInterfaces.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "CoreTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

#if !CSV_PROFILER
#error Test relies on CsvProfiler!
#endif

#if 0

CSV_DEFINE_CATEGORY(MyFalseCategory, false);
CSV_DEFINE_CATEGORY(MyTrueCategory, true);

CSV_DEFINE_CATEGORY(MyIntCategory, true);
CSV_DEFINE_CATEGORY(MyFloatCategory, true);
CSV_DEFINE_CATEGORY(MyDoubleCategory, true);

TEST_CASE("CsvProfiler")
{
	struct RAII final
	{
		RAII()
		{
			FCsvProfiler::Get()->BeginCapture();
			FCsvProfiler::Get()->BeginFrame();
		}

		~RAII()
		{
			FCsvProfiler::Get()->EndCapture();
			FCsvProfiler::Get()->EndFrame();

			// Need to double pump ending frame because of how the profiler works.
			FCsvProfiler::Get()->EndFrame();
		}
	} _;

	AutoRTFM::Testing::Abort([&]
	{
		CSV_EVENT(MyFalseCategory, TEXT("HEREWEGO"));
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_EVENT(MyFalseCategory, TEXT("HEREWEGO"));
	});

	AutoRTFM::Testing::Abort([&]
	{
		CSV_EVENT(MyTrueCategory, TEXT("HEREWEGO"));
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_EVENT(MyTrueCategory, TEXT("HEREWEGO"));
	});

	AutoRTFM::Testing::Abort([&]
	{
		CSV_EVENT(MyFalseCategory, TEXT("HEREWEGO"));
		CSV_SCOPED_TIMING_STAT(MyFalseCategory, Event);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_EVENT(MyFalseCategory, TEXT("HEREWEGO"));
		CSV_SCOPED_TIMING_STAT(MyFalseCategory, Event);
	});

	AutoRTFM::Testing::Abort([&]
	{
		CSV_EVENT(MyTrueCategory, TEXT("HEREWEGO"));
		CSV_SCOPED_TIMING_STAT(MyTrueCategory, Event);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_EVENT(MyTrueCategory, TEXT("HEREWEGO"));
		CSV_SCOPED_TIMING_STAT(MyTrueCategory, Event);
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_CUSTOM_STAT(MyFalseCategory, SomeStat, 42, ECsvCustomStatOp::Max);
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_CUSTOM_STAT(MyTrueCategory, SomeStat, 42, ECsvCustomStatOp::Max);
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_CUSTOM_STAT_GLOBAL(SomeStat, 42, ECsvCustomStatOp::Max);
	});

	AutoRTFM::Testing::Commit([&]
	{
		CSV_CUSTOM_STAT_GLOBAL(SomeStat, 42, ECsvCustomStatOp::Max);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FName Name(TEXT("Wowwee"));
		FCsvProfiler::RecordCustomStat(Name, CSV_CATEGORY_INDEX(MyIntCategory), 42, ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FCsvProfiler::RecordCustomStat("Wowwee", CSV_CATEGORY_INDEX(MyIntCategory), 42, ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FName Name(TEXT("Wowwee"));
		FCsvProfiler::RecordCustomStat(Name, CSV_CATEGORY_INDEX(MyFloatCategory), static_cast<float>(42), ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FCsvProfiler::RecordCustomStat("Wowwee", CSV_CATEGORY_INDEX(MyFloatCategory), static_cast<float>(42), ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FName Name(TEXT("Wowwee"));
		FCsvProfiler::RecordCustomStat(Name, CSV_CATEGORY_INDEX(MyDoubleCategory), static_cast<double>(42), ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FCsvProfiler::RecordCustomStat("Wowwee", CSV_CATEGORY_INDEX(MyDoubleCategory), static_cast<double>(42), ECsvCustomStatOp::Accumulate);
	});

	AutoRTFM::Testing::Commit([&]
	{
		FCsvProfiler::SetMetadata(TEXT("SomeKey"), TEXT("SomeValue"));
	});
}

TEST_CASE("CsvProfiler.EndCaptureInTransaction")
{
	SECTION("No Args")
	{
		bool bHit = false;

		FDelegateHandle Handle = FCsvProfiler::Get()->OnCSVProfileEndRequested().AddLambda([&]
		{
			REQUIRE(AutoRTFM::IsCommitting());
			bHit = true;
		});

		FCsvProfiler::Get()->BeginCapture();
		FCsvProfiler::Get()->BeginFrame();

		AutoRTFM::Testing::Commit([&]
		{
			FCsvProfiler::Get()->EndCapture();
		});

		FCsvProfiler::Get()->EndFrame();

		// Need to double pump ending frame because of how the profiler works.
		FCsvProfiler::Get()->EndFrame();

		FCsvProfiler::Get()->OnCSVProfileEndRequested().Remove(Handle);

		// TODO(SOL-8472): REQUIRE below temporarily disabled as this is flaking.
		// REQUIRE(bHit);
		(void) bHit;
	}
}

#endif // #if 0 // temp fix, also SOL-8472
