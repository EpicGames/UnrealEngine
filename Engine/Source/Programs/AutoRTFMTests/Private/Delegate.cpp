// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "Delegates/Delegate.h"
#include "MyAutoRTFMTestObject.h"
#include "UObject/GCObject.h"

TEST_CASE("Delegate.Broadcast")
{
	// Tests are sensitive to retries. Disable for these tests.
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

	TMulticastDelegate<void()> Delegate;

	int CallCount = 0;
	Delegate.AddLambda([&CallCount] AUTORTFM_OPEN
	{ 
		CallCount++;
	});

	SECTION("Transact(Broadcast)")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Delegate.Broadcast();
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
		REQUIRE(1 == CallCount);
	}

	SECTION("Transact(Open(Broadcast), Broadcast)")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			AutoRTFM::Open([&]
			{
				Delegate.Broadcast();
			});
			Delegate.Broadcast();
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
		REQUIRE(2 == CallCount);
	}

	SECTION("Transact(Broadcast, Open(Broadcast))")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Delegate.Broadcast();
			AutoRTFM::Open([&]
			{
				Delegate.Broadcast();
			});
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
		REQUIRE(2 == CallCount);
	}

	SECTION("Transact(Broadcast, Abort)")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Delegate.Broadcast();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(1 == CallCount);
	}

	SECTION("Transact(Open(Broadcast), Broadcast, Abort)")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			AutoRTFM::Open([&]
			{
				Delegate.Broadcast();
			});
			Delegate.Broadcast();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(2 == CallCount);
	}

	SECTION("Transact(Broadcast, Open(Broadcast), Abort)")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Delegate.Broadcast();
			AutoRTFM::Open([&]
			{
				Delegate.Broadcast();
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(2 == CallCount);
	}

	SECTION("AddStatic(Abort) Transact(Broadcast)")
	{
		Delegate.AddStatic([] AUTORTFM_ENABLE { AutoRTFM::AbortTransaction(); });
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Delegate.Broadcast();
		});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(0 == CallCount);
	}
}

TEST_CASE("Delegate.Recursive")
{
	SECTION("Open <-> Close")
	{
		TMulticastDelegate<void(int, int&)> Delegate;

		Delegate.AddLambda([&] AUTORTFM_ENABLE (int X, int& R)
		{
			REQUIRE(AutoRTFM::IsTransactional());

			if (AutoRTFM::IsClosed())
			{
				AutoRTFM::Open([&]
				{
					if (X <= 1)
					{
						R = X;
					}
					else
					{
						int R1, R2;
						Delegate.Broadcast(X - 1, R1);
						Delegate.Broadcast(X - 2, R2);
						R = R1 + R2;
					}
				});
			}
			else
			{
				(void)AutoRTFM::Close([&]
				{
					if (X <= 1)
					{
						R = X;
					}
					else
					{
						int R1, R2;
						Delegate.Broadcast(X - 1, R1);
						Delegate.Broadcast(X - 2, R2);
						R = R1 + R2;
					}
				});
			}
		});

		AutoRTFM::Testing::Commit([&]
		{
			int R = 0;
			Delegate.Broadcast(6, R);
			REQUIRE(R == 8);
		});

		Delegate.AddStatic([](int, int&) {});
	}

	
	SECTION("Open <-> Close With Abort")
	{
		TMulticastDelegate<void(int, int&, bool&)> Delegate;
		TMulticastDelegate<void(int, int&, bool&)> Other;

		Delegate.AddLambda([&] AUTORTFM_ENABLE (int X, int& R, bool& bAborting)
		{
			REQUIRE(AutoRTFM::IsTransactional());

			if (X <= 1)
			{
				R = X;

				const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
				{
					AutoRTFM::AbortTransaction();	
				});
				
				bAborting = AutoRTFM::HasAborted(Status);
			}
			else if (AutoRTFM::IsClosed())
			{
				AutoRTFM::Open([&]
				{
					int R1, R2;
					Delegate.Broadcast(X - 1, R1, bAborting);

					if (bAborting)
					{
						return;
					}

					Delegate.Broadcast(X - 2, R2, bAborting);

					if (bAborting)
					{
						return;
					}

					R = R1 + R2;
				});
			}
			else
			{
				const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
				{
					int R1, R2;
					Delegate.Broadcast(X - 1, R1, bAborting);
					Delegate.Broadcast(X - 2, R2, bAborting);
					R = R1 + R2;
				});

				bAborting = AutoRTFM::HasAborted(Status);
			}
		});

		AutoRTFM::Testing::Abort([&]
		{
			bool bAborting = false;
			int R = 0;
			Delegate.Broadcast(6, R, bAborting);
			FAIL("Unreachable!");
		});

		AutoRTFM::Testing::Abort([&]
		{
			bool bAborting = false;
			int R = 0;
			Delegate.Broadcast(6, R, bAborting);
			Other = MoveTemp(Delegate);
			FAIL("Unreachable!");
		});

		AutoRTFM::Testing::Abort([&]
		{
			bool bAborting = false;
			int R = 0;
			Delegate.Broadcast(5, R, bAborting);
			FAIL("Unreachable!");
		});

		AutoRTFM::Testing::Abort([&]
		{
			bool bAborting = false;
			int R = 0;
			Delegate.Broadcast(5, R, bAborting);
			Other = MoveTemp(Delegate);
			FAIL("Unreachable!");
		});

		Other = MoveTemp(Delegate);
	}
}

TEST_CASE("Delegate.CauseCompaction")
{
	// Tests are sensitive to retries. Disable for these tests.
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

	SECTION("Transact")
	{
		TMulticastDelegate<void()> Delegate;

		FDelegateHandle Handle = Delegate.AddStatic([]() { FAIL("Unreachable!"); });

		bool bHitOnce = false;

		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			if (!bHitOnce)
			{
				REQUIRE(Delegate.Remove(Handle));
				bHitOnce = true;
			}
		});

		AutoRTFM::Testing::Commit([&] { Delegate.Broadcast(); });

		Delegate.Broadcast();
	}

	SECTION("OnCommit")
	{
		TMulticastDelegate<void()> Delegate;

		FDelegateHandle Handle = Delegate.AddStatic([]() { FAIL("Unreachable!"); });

		bool bHitOnce = false;

		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			if (!bHitOnce)
			{
				REQUIRE(Delegate.Remove(Handle));
				bHitOnce = true;
			}
		});

		AutoRTFM::Testing::Commit([&] { AutoRTFM::OnCommit([&] { Delegate.Broadcast(); }); });

		Delegate.Broadcast();
	}
	
	SECTION("OnAbort")
	{
		TMulticastDelegate<void()> Delegate;

		FDelegateHandle Handle = Delegate.AddStatic([]() { FAIL("Unreachable!"); });

		bool bHitOnce = false;

		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			if (!bHitOnce)
			{
				REQUIRE(Delegate.Remove(Handle));
				bHitOnce = true;
			}
		});

		AutoRTFM::Testing::Abort([&] { AutoRTFM::OnAbort([&] { Delegate.Broadcast(); }); AutoRTFM::AbortTransaction(); });

		Delegate.Broadcast();
	}
}

TEST_CASE("Delegate.RemoveAll")
{
	// Tests are sensitive to retries. Disable for these tests.
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

	SECTION("Transact")
	{
		TMulticastDelegate<void()> Delegate;
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		bool bExecutedOnce = false;
		
		Delegate.AddUObject(Object, &UMyAutoRTFMTestObject::DoNothing);
		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			REQUIRE(!bExecutedOnce);
			bExecutedOnce = true;
			Delegate.RemoveAll(Object);
		});
			
		REQUIRE(Delegate.IsBoundToObject(Object));

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Delegate.IsBoundToObject(Object));
			Delegate.Broadcast();
			REQUIRE(!Delegate.IsBoundToObject(Object));
		});

		REQUIRE(!Delegate.IsBoundToObject(Object));
	}
	
	SECTION("OnCommit")
	{
		TMulticastDelegate<void()> Delegate;
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		bool bExecutedOnce = false;
		
		Delegate.AddUObject(Object, &UMyAutoRTFMTestObject::DoNothing);
		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			REQUIRE(!bExecutedOnce);
			bExecutedOnce = true;
			Delegate.RemoveAll(Object);
		});
			
		REQUIRE(Delegate.IsBoundToObject(Object));

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnCommit([&]
			{
				REQUIRE(Delegate.IsBoundToObject(Object));
				Delegate.Broadcast();
				REQUIRE(!Delegate.IsBoundToObject(Object));
			});
		});

		REQUIRE(!Delegate.IsBoundToObject(Object));
	}

	SECTION("OnAbort")
	{
		TMulticastDelegate<void()> Delegate;
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		bool bExecutedOnce = false;
		
		Delegate.AddUObject(Object, &UMyAutoRTFMTestObject::DoNothing);
		Delegate.AddLambda([&] AUTORTFM_ENABLE
		{
			REQUIRE(!bExecutedOnce);
			bExecutedOnce = true;
			Delegate.RemoveAll(Object);
		});
			
		REQUIRE(Delegate.IsBoundToObject(Object));

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([&]
			{
				REQUIRE(Delegate.IsBoundToObject(Object));
				Delegate.Broadcast();
				REQUIRE(!Delegate.IsBoundToObject(Object));
			});

			AutoRTFM::AbortTransaction();
		});

		REQUIRE(!Delegate.IsBoundToObject(Object));
	}
}
