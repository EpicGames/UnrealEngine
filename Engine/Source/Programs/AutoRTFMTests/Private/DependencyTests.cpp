// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"

// We can't use Catch2Includes.h here, because it #includes more Unreal headers.
// Fortunately, we don't even need to run these tests; if this code compiles and links, 
// the test has passed.

namespace 
{

void OnAbort_MacroWorksWithNoAdditionalIncludes()
{
	AutoRTFM::Transact([&]
	{
		UE_AUTORTFM_ONABORT()
		{
		};
	});
}

void OnAbort_FunctionWorksWithNoAdditionalIncludes()
{
	AutoRTFM::Transact([&]
	{
		AutoRTFM::OnAbort([]
		{
		});
	});
}

void OnCommit_MacroWorksWithNoAdditionalIncludes()
{
	AutoRTFM::Testing::Commit([&]
	{
		UE_AUTORTFM_ONCOMMIT()
		{
		};
	});
}

void OnCommit_FunctionWorksWithNoAdditionalIncludes()
{
	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::OnCommit([]
		{
		});
	});
}

}  // namespace
