// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "BuildMacros.h"

#include <string>
#include <vector>

#if AUTORTFM_EXCEPTIONS_ENABLED

TEST_CASE("Exceptions")
{
	// Tests are sensitive to retries. Disable for these tests.
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

	using EventList = std::vector<std::string>;
	EventList Events;

	auto Event = [&Events] AUTORTFM_ENABLE (std::string_view Event)
	{
		UE_AUTORTFM_OPEN
		{
			Events.push_back(std::string(Event));
		};
	};

	struct AUTORTFM_OPEN FScope
	{
		FScope(EventList& Events, const char* Name) : Events{Events}, Name{Name}
		{
			Events.push_back(std::string{Name});
		}
		~FScope()
		{
			Events.push_back(std::string{"End-"} + Name);
		}
		EventList& Events;
		char const* const Name;
	};
#define SCOPE(INDEX) FScope UE_AUTORTFM_CONCAT(Scope_, __COUNTER__){Events, INDEX}

	SECTION("Try(Transact())")
	{
		{
			SCOPE("Outer");
			try
			{
				SCOPE("Try");
				AutoRTFM::Testing::Commit([&]
				{
					SCOPE("Transact");
				});
			}
			catch (...)
			{
				SCOPE("Catch");
			}
		}

		const EventList Expected
		{
			"Outer",
			"Try",
			"Transact",
			"End-Transact",
			"End-Try",
			"End-Outer",
		};
		REQUIRE(Events == Expected);
	}

	SECTION("Try(Transact(Abort))")
	{
		{
			SCOPE("Outer");
			try
			{
				SCOPE("Try");
				AutoRTFM::Testing::Abort([&]
				{
					SCOPE("Transact");
					Event("Abort");
					AutoRTFM::AbortTransaction();
				});
			}
			catch (...)
			{
				SCOPE("Catch");
			}
		}

		const EventList Expected
		{
			"Outer",
			"Try",
			"Transact",
			"Abort",
			"End-Try",
			"End-Outer",
		};
		REQUIRE(Events == Expected);
	}

	SECTION("Try(Transact(Throw))")
	{
		{
			SCOPE("Outer");
			try
			{
				SCOPE("Try");
				AutoRTFM::Testing::Commit([&]
				{
					SCOPE("Transact");
					Event("Throw");
					throw(42);
				});
			}
			catch (int I)
			{
				SCOPE("Catch");
				REQUIRE(I == 42);
			}
		}

		const EventList Expected
		{
			"Outer",
			"Try",
			"Transact",
			"Throw",
			"End-Transact",
			"End-Try",
			"Catch",
			"End-Catch",
			"End-Outer",
		};
		REQUIRE(Events == Expected);
	}

	SECTION("Transact(Try(Throw))")
	{
		{
			SCOPE("Outer");
			AutoRTFM::Testing::Commit([&]
			{
				SCOPE("Transact");
				try
				{
					SCOPE("Try");
					Event("Throw");
					throw(42);
				}
				catch (int I)
				{
					SCOPE("Catch");
					REQUIRE(I == 42);
				}
			});
		}

		const EventList Expected
		{
			"Outer",
			"Transact",
			"Try",
			"Throw",
			"End-Try",
			"Catch",
			"End-Catch",
			"End-Transact",
			"End-Outer",
		};
		REQUIRE(Events == Expected);
	}

	SECTION("Transact(Try(Transact(Throw)))")
	{
		{
			SCOPE("Outer");
			AutoRTFM::Testing::Commit([&]
			{
				SCOPE("Transact-A");
				try
				{
					SCOPE("Try");
					AutoRTFM::Testing::Commit([&]
					{
						SCOPE("Transact-B");
						Event("Throw");
						throw(42);
					});
				}
				catch (int I)
				{
					SCOPE("Catch");
					REQUIRE(I == 42);
				}
			});
		}

		const EventList Expected
		{
			"Outer",
			"Transact-A",
			"Try",
			"Transact-B",
			"Throw",
			"End-Transact-B",
			"End-Try",
			"Catch",
			"End-Catch",
			"End-Transact-A",
			"End-Outer",
		};
		REQUIRE(Events == Expected);
	}
}

// ---------------------------------------------------------------------------
// Mode-observation + abort-revert tests for destructors during exception unwind.
//
// Each test creates a function in a specific mode that:
// 1. Constructs an RAII object whose destructor writes to memory and records
//    the observed mode (IsClosed).
// 2. Throws an exception.
//
// The caller wraps this in try/catch inside a transaction, then aborts.
// We verify:
// - The destructor ran (the write happened).
// - The destructor observed the correct mode (IsClosed).
// - For closed-mode destructors: the write IS reverted on abort.
// - For open-mode destructors: the write is NOT reverted on abort.
// ---------------------------------------------------------------------------

namespace
{

struct FModeRecorder
{
	int* WriteTarget;
	bool* ObservedIsClosed;

	~FModeRecorder()
	{
		if (WriteTarget)
		{
			*WriteTarget = 99;
		}
		if (ObservedIsClosed)
		{
			*ObservedIsClosed = AutoRTFM::IsClosed();
		}
	}
};

// Each function constructs an FModeRecorder locally so the destructor
// runs inside the function's mode during exception unwinding.

AUTORTFM_OPEN static void OpenThenThrow(int* WriteTarget, bool* ObservedIsClosed)
{
	FModeRecorder Recorder{WriteTarget, ObservedIsClosed};
	throw 42;
}

AUTORTFM_OPEN_NO_VALIDATION static void OpenNoSanitizeThenThrow(int* WriteTarget, bool* ObservedIsClosed)
{
	FModeRecorder Recorder{WriteTarget, ObservedIsClosed};
	throw 42;
}

[[clang::autortfm(autortfm_mode_internal)]] static void InternalThenThrow(int* WriteTarget, bool* ObservedIsClosed)
{
	FModeRecorder Recorder{WriteTarget, ObservedIsClosed};
	throw 42;
}

// Disable-mode throwing function. Called via OpenCallDisableThenThrow
// to avoid the compile error of calling a disabled function from enabled code.
AUTORTFM_DISABLE static void DisableThenThrow(int* WriteTarget, bool* ObservedIsClosed)
{
	FModeRecorder Recorder{WriteTarget, ObservedIsClosed};
	throw 42;
}

AUTORTFM_OPEN static void OpenCallDisableThenThrow(int* WriteTarget, bool* ObservedIsClosed)
{
	DisableThenThrow(WriteTarget, ObservedIsClosed);
}

UE_AUTORTFM_ALWAYS_OPEN static void AssignIntPointer(int* Pointer, int Value)
{
	*Pointer = Value;
}

} // anonymous namespace

TEST_CASE("Exceptions.DestructorModes")
{
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

	SECTION("Destructor in enable mode: closed=true")
	{
		int WriteTarget = 0;
		bool ObservedIsClosed = false;

		AutoRTFM::Transact([&]
		{
			try
			{
				// FModeRecorder constructed inline in the enable-mode lambda.
				// Its destructor runs in enable mode (closed) during unwind.
				FModeRecorder Recorder{&WriteTarget, &ObservedIsClosed};
				throw 42;
			}
			catch (...) {}
		});

		REQUIRE(ObservedIsClosed == true);
		// The destructor wrote 99 in closed code, and the transaction
		// committed, so the write is visible.
		REQUIRE(WriteTarget == 99);
	}

	SECTION("Destructor in open mode: closed=false, write NOT reverted on abort")
	{
		int WriteTarget = 0;
		bool ObservedIsClosed = false;

		AutoRTFM::Testing::Abort([&]
		{
			try
			{
				// FModeRecorder constructed inside OpenThenThrow (open mode).
				// Its destructor runs in open mode during unwind.
				OpenThenThrow(&WriteTarget, &ObservedIsClosed);
			}
			catch (...) {}
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(ObservedIsClosed == false);
		REQUIRE(WriteTarget == 99); // NOT reverted (open write).
	}

	SECTION("Destructor in open_no_sanitize mode: closed=false, write NOT reverted on abort")
	{
		int WriteTarget = 0;
		bool ObservedIsClosed = false;

		AutoRTFM::Testing::Abort([&]
		{
			try
			{
				OpenNoSanitizeThenThrow(&WriteTarget, &ObservedIsClosed);
			}
			catch (...) {}
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(ObservedIsClosed == false);
		REQUIRE(WriteTarget == 99); // NOT reverted (open write).
	}

	SECTION("Destructor in internal mode: closed=false, write NOT reverted on abort")
	{
		int WriteTarget = 0;
		bool ObservedIsClosed = false;

		AutoRTFM::Testing::Abort([&]
		{
			try
			{
				InternalThenThrow(&WriteTarget, &ObservedIsClosed);
			}
			catch (...) {}
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(ObservedIsClosed == false);
		REQUIRE(WriteTarget == 99); // NOT reverted (internal = open write).
	}

	SECTION("Destructor in disable mode: closed=false, write NOT reverted on abort")
	{
		int WriteTarget = 0;
		bool ObservedIsClosed = false;

		AutoRTFM::Testing::Abort([&]
		{
			try
			{
				// Call via an open wrapper to avoid compile error
				// (disabled functions can't be called from enabled code).
				OpenCallDisableThenThrow(&WriteTarget, &ObservedIsClosed);
			}
			catch (...) {}
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(ObservedIsClosed == false);
		REQUIRE(WriteTarget == 99); // NOT reverted (disable = open write).
	}
}

#if AUTORTFM_SANITIZER
TEST_CASE("Exceptions.Sanitizer")
{
	AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);
	AUTORTFM_SCOPED_SANITIZER_MODE_WARN();

	SECTION("Sanitizer re-enabled after exception through open_no_sanitize")
	{
		AutoRTFMTestUtils::FCaptureWarningContext WarningContext;
		int I = 0;

		AutoRTFM::Transact([&]
		{
			I = 42; // Closed write.
			try
			{
				OpenNoSanitizeThenThrow(nullptr, nullptr);
			}
			catch (...) {}

			// After the catch, the sanitizer MUST be re-enabled.
			// This open write conflicts with I = 42, so it should warn.
			AssignIntPointer(&I, 24);
		});

		REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
	}
}
#endif // AUTORTFM_SANITIZER

#endif // AUTORTFM_EXCEPTIONS_ENABLED
