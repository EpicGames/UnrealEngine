// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"

#include <vector>

enum class EOpcodeType
{
	StartTransaction,
	CommitTransaction,
	Call,
	WriteToValue
};

typedef void (*FCall)(int*, int);

struct FOpcode final
{
private:
	explicit FOpcode(EOpcodeType Type, int Value = -1, FCall Call = nullptr) : Type(Type), Value(Value), Call(Call) {}

public:
	EOpcodeType Type;
	int Value;
	FCall Call;

	static FOpcode CreateStartTransaction(int RecoveryOpcodeOffset)
	{
		return FOpcode(EOpcodeType::StartTransaction, RecoveryOpcodeOffset);
	}

	static FOpcode CreateCommitTransaction()
	{
		return FOpcode(EOpcodeType::CommitTransaction);
	}

	static FOpcode CreateCall(int Arg, FCall Call)
	{
		return FOpcode(EOpcodeType::Call, Arg, Call);
	}

	static FOpcode CreateWriteToValue(int Arg)
	{
		return FOpcode(EOpcodeType::WriteToValue, Arg);	
	}
};

void Good(int* Value, int Arg)
{
	*Value += Arg;
}

void Bad(int*, int)
{
	AutoRTFM::AbortTransaction();
}

AutoRTFM::ETransactionResult FakeVM(const FOpcode* Opcodes, const int Length, int& Value)
{
	// SOL-6743: These tests perform writes within open and closed scopes to the same memory
	// locations, which will cause AutoRTFM sanitizer errors.
	AUTORTFM_SCOPED_SANITIZER_MODE_DISABLE();

	return AutoRTFM::Transact([Opcodes, Length, &Value]
	{
		AutoRTFM::Open([Opcodes, Length, &Value] AUTORTFM_DISABLE
		{
			std::vector<int> RecoveryOpcodeStack;

			for (int i = 0; i < Length; i++)
			{
				switch (Opcodes[i].Type)
				{
				default:
					FAIL("Unhandled opcode kind!");
					break;
				case EOpcodeType::StartTransaction:
					AutoRTFM::StartTransaction();
					RecoveryOpcodeStack.push_back(Opcodes[i].Value);
					break;
				case EOpcodeType::CommitTransaction:
					AutoRTFM::CommitTransaction();
					RecoveryOpcodeStack.pop_back();
					break;
				case EOpcodeType::Call:
				{
					const FCall Call = Opcodes[i].Call;
					const int Arg = Opcodes[i].Value;

					const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&Value, Arg, Call]
						{
							Call(&Value, Arg);
						});

					if (AutoRTFM::HasAborted(Status))
					{
						if (RecoveryOpcodeStack.empty())
						{
							// We're wanting to throw out to the parent Transact call!
							return;
						}

						// -1 just cause the loop will i++!
						i = RecoveryOpcodeStack.back() - 1;
						RecoveryOpcodeStack.pop_back();
					}
					break;
				}
				case EOpcodeType::WriteToValue:
				{
					const int Arg = Value - Opcodes[i].Value;
					AutoRTFM::RecordOpenWrite(&Value);
					Value = Arg;
					break;
				}
				}
			}
		});
	});
}

TEST_CASE("FakeVM.NativeCallInTransaction")
{
	constexpr int Length = 6;
	const FOpcode Opcodes[Length] =
	{
		// 5 being the offset just after the commit
		FOpcode::CreateStartTransaction(5),
		FOpcode::CreateWriteToValue(42),
		FOpcode::CreateCall(43, &Good),
		FOpcode::CreateWriteToValue(13),
		FOpcode::CreateCommitTransaction(),
		FOpcode::CreateWriteToValue(-53),
	};

	int Value = 100;
	REQUIRE(AutoRTFM::ETransactionResult::Committed == FakeVM(Opcodes, Length, Value));
	REQUIRE(Value == (100 - 42 + 43 - 13 - (-53)));
}

TEST_CASE("FakeVM.AbortInTransaction")
{
	constexpr int Length = 6;
	const FOpcode Opcodes[Length] =
	{
		// 5 being the offset just after the commit
		FOpcode::CreateStartTransaction(5),
		FOpcode::CreateWriteToValue(42),
		FOpcode::CreateCall(43, &Bad),
		FOpcode::CreateWriteToValue(13),
		FOpcode::CreateCommitTransaction(),
		FOpcode::CreateWriteToValue(-53),
	};

	int Value = 100;
	REQUIRE(AutoRTFM::ETransactionResult::Committed == FakeVM(Opcodes, Length, Value));
	REQUIRE(Value == (100 - (-53)));
}

TEST_CASE("FakeVM.Abort")
{
	constexpr int Length = 3;
	const FOpcode Opcodes[Length] =
	{
		FOpcode::CreateWriteToValue(42),
		FOpcode::CreateCall(43, &Bad),
		FOpcode::CreateWriteToValue(13),
	};

	int Value = 100;
	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == FakeVM(Opcodes, Length, Value));
	REQUIRE(Value == 100);
}
