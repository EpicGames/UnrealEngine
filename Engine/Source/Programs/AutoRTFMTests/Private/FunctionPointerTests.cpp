// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "Templates/Function.h"

#include <cmath>

namespace
{
	int TestCFunction()
	{
		if (AutoRTFM::IsClosed())
		{
			return 42;
		}
		else
		{
			AutoRTFM::AbortTransaction();
		}

		return 43;
	}

	using CStyleType = int (*)();
	using CosfType = float (*)(float);

	UE_DISABLE_OPTIMIZATION_SHIP
	CStyleType GetTestCFunction()
	{
		return &TestCFunction;
	}

	CosfType GetCosfFunction()
	{
		return &std::cosf;
	}
	UE_ENABLE_OPTIMIZATION_SHIP
}

TEST_CASE("FunctionPointer.CStyle")
{
	AutoRTFM::Testing::Commit([&]
		{
			CStyleType CStyle = GetTestCFunction();
			REQUIRE(42 == CStyle());
		});
}

TEST_CASE("FunctionPointer.StandardLibrary")
{
	SECTION("Created inside transaction")
	{
		AutoRTFM::Testing::Commit([&]
			{
				CosfType fn = GetCosfFunction();
				REQUIRE(1.0f == fn(0.0f));
			});
	}

	SECTION("Created outside transaction")
	{
		CosfType fn = GetCosfFunction();

		AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(1.0f == fn(0.0f));
			});
	}
}

TEST_CASE("FunctionPointer.TFunction")
{
	SECTION("Created inside transaction")
	{
		int Result = 0;
		AutoRTFM::Testing::Commit([&]
			{
				TFunction<void()> MyFunc = [&Result]
					{
						Result = 42;
					};

				if (MyFunc)
				{
					MyFunc();
				}

				check(MyFunc);

				MyFunc.Reset();
			});

		REQUIRE(42 == Result);
	}

	SECTION("Created outside transaction")
	{
		int Result = 0;
		TFunction<void()> MyFunc = [&Result] AUTORTFM_ENABLE
			{
				Result = 42;
			};

		AutoRTFM::Testing::Commit([&]
			{
				if (MyFunc)
				{
					MyFunc();
				}

				check(MyFunc);

				MyFunc.Reset();
			});

		REQUIRE(42 == Result);
	}
}

TEST_CASE("FunctionPointer.TUniqueFunction")
{
	SECTION("Created inside transaction")
	{
		int Result = 0;
		AutoRTFM::Testing::Commit([&]
			{
				TUniqueFunction<void()> MyFunc = [&Result]
					{
						Result = 42;
					};

				if (MyFunc)
				{
					MyFunc();
				}

				check(MyFunc);

				MyFunc.Reset();
			});

		REQUIRE(42 == Result);
	}

	SECTION("Created outside transaction")
	{
		int Result = 0;
		TUniqueFunction<void()> MyFunc = [&Result] AUTORTFM_ENABLE
			{
				Result = 42;
			};

		AutoRTFM::Testing::Commit([&]
			{
				if (MyFunc)
				{
					MyFunc();
				}

				check(MyFunc);

				MyFunc.Reset();
			});

		REQUIRE(42 == Result);
	}
}
