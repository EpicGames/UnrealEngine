// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "Catch2Includes.h"
#include "Hash/Blake3.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/MemoryFwd.h"

TEST_CASE("Hash.Blake3")
{
	SECTION("Construct / Destruct")
	{
		SECTION("With Abort")
		{
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				FBlake3 Hash;
				AutoRTFM::AbortTransaction();
			});
			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		}
		SECTION("With Commit")
		{
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				FBlake3 Hash;
			});
			REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
		}
	}

	SECTION("Update")
	{
		const FBlake3Hash ExpectedCommitHash{FUtf8StringView("33f1cb24398ef72a663a8aa5afe2bef9c6d5ff2490e457201c3113d333642627")};
		const FBlake3Hash ExpectedAbortHash{FUtf8StringView("af1349b9f5f9a1a6a0404dea36dcc9499bcb25c9adc112b7cc9a93cae41f3262")};
		const char Data[] = "Hello world";

		FBlake3 Hash;

		SECTION("Ptr + Size")
		{
			SECTION("With Abort")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(Data, sizeof(Data));
					AutoRTFM::AbortTransaction();
				});
				REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
				REQUIRE(ExpectedAbortHash == Hash.Finalize());
			}
			SECTION("With Commit")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(Data, sizeof(Data));
				});
				REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
				REQUIRE(ExpectedCommitHash == Hash.Finalize());
			}
		}

		SECTION("FMemoryView")
		{
			FMemoryView View(Data, sizeof(Data));
			SECTION("With Abort")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(View);
					AutoRTFM::AbortTransaction();
				});
				REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
				REQUIRE(ExpectedAbortHash == Hash.Finalize());
			}
			SECTION("With Commit")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(View);
				});
				REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
				REQUIRE(ExpectedCommitHash == Hash.Finalize());
			}
		}

		SECTION("FCompositeBuffer")
		{
			FCompositeBuffer Buffer{FSharedBuffer::MakeView(Data, sizeof(Data))};
			SECTION("With Abort")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(Buffer);
					AutoRTFM::AbortTransaction();
				});
				REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
				REQUIRE(ExpectedAbortHash == Hash.Finalize());
			}
			SECTION("With Commit")
			{
				AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
				{
					Hash.Update(Buffer);
				});
				REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
				REQUIRE(ExpectedCommitHash == Hash.Finalize());
			}
		}
	}

	SECTION("Finalize")
	{
		const FBlake3Hash ExpectedCommitHash{FUtf8StringView("33f1cb24398ef72a663a8aa5afe2bef9c6d5ff2490e457201c3113d333642627")};
		const FBlake3Hash ExpectedAbortHash{FUtf8StringView("0000000000000000000000000000000000000000000000000000000000000000")};
		const char Data[] = "Hello world";

		FBlake3 Hash;
		Hash.Update(Data, sizeof(Data));

		FBlake3Hash Got = FBlake3Hash::Zero;

		SECTION("With Abort")
		{
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Got = Hash.Finalize();
				AutoRTFM::AbortTransaction();
			});
			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
			REQUIRE(ExpectedAbortHash == Got);
		}
		SECTION("With Commit")
		{
			AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Got = Hash.Finalize();
			});
			REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
			REQUIRE(ExpectedCommitHash == Got);
		}
	}
}
