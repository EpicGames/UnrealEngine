// Copyright Epic Games, Inc. All Rights Reserved.

#include "HashMap.h"
#include "Catch2Includes.h"
#include "ObjectLifetimeHelper.h"
#include <unordered_map>

using HashMapIntToInt = AutoRTFM::THashMap<int, int>;
using HashMapIntToObj = AutoRTFM::THashMap<int, AutoRTFMTestUtils::FObjectLifetimeHelper>;
using HashMapObjToInt = AutoRTFM::THashMap<AutoRTFMTestUtils::FObjectLifetimeHelper, int>;
using HashMapObjToObj = AutoRTFM::THashMap<AutoRTFMTestUtils::FObjectLifetimeHelper, AutoRTFMTestUtils::FObjectLifetimeHelper>;

TEMPLATE_TEST_CASE("HashMap", "", HashMapIntToInt, HashMapIntToObj, HashMapObjToInt, HashMapObjToObj)
{
	AutoRTFM::UnreachableIfClosed();

	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == 0);
	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls == 0);

	using MapType = TestType;
	using KeyType = typename TestType::Key;
	using ValueType = typename TestType::Value;

	auto Check = [&] AUTORTFM_DISABLE (MapType& Map, const std::unordered_map<KeyType, ValueType>& Expected)
	{
		// Check the reported count is as expected.
		REQUIRE(Map.Num() == Expected.size());
		REQUIRE(Map.IsEmpty() == Expected.empty());

		// Check the non-const begin() / end().
		{
			size_t I = 0;
			for (AutoRTFM::TKeyAndValue<const KeyType, ValueType>& Item : Map)
			{
				I++;
				auto ExpectedIt = Expected.find(Item.Key);
				REQUIRE(ExpectedIt != Expected.end());
				REQUIRE(ExpectedIt->second == Item.Value);
				// Check mutability of the value
				Item.Value = 99;
				REQUIRE(Item.Value == 99);
				Item.Value = ExpectedIt->second;
			}
			REQUIRE(I == Expected.size());
		}
		// Check the const begin() / end().
		{
			size_t I = 0;
			for (const AutoRTFM::TKeyAndValue<const KeyType, ValueType>& Item : const_cast<const MapType&>(Map))
			{
				I++;
				auto ExpectedIt = Expected.find(Item.Key);
				REQUIRE(ExpectedIt != Expected.end());
				REQUIRE(ExpectedIt->second == Item.Value);
			}
			REQUIRE(I == Expected.size());
		}
		// Check Contains() & Find() for all elements in Expected.
		{
			for (auto It : Expected)
			{
				REQUIRE(Map.Contains(It.first));
				ValueType* FindValue = Map.Find(It.first);
				REQUIRE(FindValue);
				REQUIRE(*FindValue == It.second);
			}
		}
	};

	SECTION("Add")
	{
		MapType Map;
		Check(Map, {});

		Map.Add(10, 100);
		Check(Map, {{10, 100}});

		Map.Add(20, 200);
		Check(Map, {{10, 100}, {20, 200}});

		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		Map.Add(20, 123);
		Check(Map, {{10, 100}, {20, 123}, {30, 300}});

		Map.Add(10, 321);
		Check(Map, {{10, 321}, {20, 123}, {30, 300}});

		Map.Add(40, 400);
		Check(Map, {{10, 321}, {20, 123}, {30, 300}, {40, 400}});
	}

	SECTION("Find")
	{
		MapType Map;
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);

		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		{
			ValueType* Value = Map.Find(KeyType{0});
			REQUIRE(!Value);
		}
		{
			ValueType* Value = Map.Find(KeyType{10});
			REQUIRE((Value && *Value == 100));
		}
		{
			ValueType* Value = Map.Find(KeyType{20});
			REQUIRE((Value && *Value == 200));
		}
		{
			ValueType* Value = Map.Find(KeyType{30});
			REQUIRE((Value && *Value == 300));
		}
		{
			ValueType* Value = Map.Find(KeyType{40});
			REQUIRE(!Value);
		}

		Check(Map, {{10, 100}, {20, 200}, {30, 300}});
	}

	SECTION("FindOrAdd")
	{
		MapType Map;
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		REQUIRE(Map.FindOrAdd(0) == 0);
		Check(Map, {{0, 0}, {10, 100}, {20, 200}, {30, 300}});
		REQUIRE(Map.FindOrAdd(10) == 100);
		Check(Map, {{0, 0}, {10, 100}, {20, 200}, {30, 300}});
		REQUIRE(Map.FindOrAdd(20) == 200);
		Check(Map, {{0, 0}, {10, 100}, {20, 200}, {30, 300}});
		REQUIRE(Map.FindOrAdd(30) == 300);
		Check(Map, {{0, 0}, {10, 100}, {20, 200}, {30, 300}});
		REQUIRE(Map.FindOrAdd(40) == 0);
		Check(Map, {{0, 0}, {10, 100}, {20, 200}, {30, 300}, {40, 0}});
	}

	SECTION("Remove")
	{
		MapType Map;
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		Map.Remove(0);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});
		Map.Remove(10);
		Check(Map, {{20, 200}, {30, 300}});
		Map.Remove(20);
		Check(Map, {{30, 300}});
		Map.Remove(30);
		Check(Map, {});
		Map.Remove(40);
		Check(Map, {});
	}

	SECTION("Contains")
	{
		MapType Map;
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		REQUIRE(Map.Contains(0) == false);
		REQUIRE(Map.Contains(10) == true);
		REQUIRE(Map.Contains(20) == true);
		REQUIRE(Map.Contains(30) == true);
		REQUIRE(Map.Contains(40) == false);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});
	}

	SECTION("Empty")
	{
		MapType Map;
		Check(Map, {});

		Map.Empty();
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		Map.Empty();
		Check(Map, {});
	}

	SECTION("Reset")
	{
		MapType Map;
		Check(Map, {});

		Map.Reset();
		Check(Map, {});

		Map.Add(10, 100);
		Map.Add(20, 200);
		Map.Add(30, 300);
		Check(Map, {{10, 100}, {20, 200}, {30, 300}});

		Map.Reset();
		Check(Map, {});
	}

	SECTION("Copy Construct")
	{
		SECTION("Empty")
		{
			MapType Source;
			MapType Target(Source);
			Check(Source, {});
			Check(Target, {});
		}

		SECTION("Non-empty")
		{
			MapType Source;
			Source.Add(10, 100);
			Source.Add(20, 200);
			Source.Add(30, 300);
			MapType Target(Source);
			Check(Source, {{10, 100}, {20, 200}, {30, 300}});
			Check(Target, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Move Construct")
	{
		SECTION("Empty")
		{
			MapType Source;
			MapType Target(std::move(Source));
			Check(Source, {});
			Check(Target, {});
		}

		SECTION("Non-empty")
		{
			MapType Source;
			Source.Add(10, 100);
			Source.Add(20, 200);
			Source.Add(30, 300);
			MapType Target(std::move(Source));
			Check(Source, {});
			Check(Target, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Copy Assign")
	{
		SECTION("Empty")
		{
			MapType Source;
			MapType Target;
			Target = Source;
			Check(Source, {});
			Check(Target, {});
		}

		SECTION("Non-empty")
		{
			MapType Source;
			Source.Add(10, 100);
			Source.Add(20, 200);
			Source.Add(30, 300);
			MapType Target;
			Target = Source;
			Check(Source, {{10, 100}, {20, 200}, {30, 300}});
			Check(Target, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Copy Assign Self")
	{
		SECTION("Empty")
		{
			MapType Map;
			MapType& Alias = Map;
			Map = Alias;
			Check(Map, {});
		}

		SECTION("Non-empty")
		{
			MapType Map;
			MapType& Alias = Map;
			Map.Add(10, 100);
			Map.Add(20, 200);
			Map.Add(30, 300);
			Map = Alias;
			Check(Map, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Move Assign")
	{
		SECTION("Empty")
		{
			MapType Source;
			MapType Target;
			Target = std::move(Source);
			Check(Source, {});
			Check(Target, {});
		}

		SECTION("Non-empty")
		{
			MapType Source;
			Source.Add(10, 100);
			Source.Add(20, 200);
			Source.Add(30, 300);
			MapType Target;
			Target = std::move(Source);
			Check(Source, {});
			Check(Target, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Move Assign Self")
	{
		SECTION("Empty")
		{
			MapType Map;
			MapType& Alias = Map;
			Map = std::move(Alias);
			Check(Map, {});
		}

		SECTION("Non-empty")
		{
			MapType Map;
			MapType& Alias = Map;
			Map.Add(10, 100);
			Map.Add(20, 200);
			Map.Add(30, 300);
			Map = std::move(Alias);
			Check(Map, {{10, 100}, {20, 200}, {30, 300}});
		}
	}

	SECTION("Soak")
	{
		MapType Map;
		std::unordered_map<KeyType, ValueType> Expected;
		for (unsigned I = 0; I < 10000; I++)
		{
			unsigned Mod100 = (I * 15485863) % 100;
			KeyType Key = KeyType{(I * 804889) % 1000};
			switch (Mod100)
			{
				case 0:
				{
					ValueType* Value = Map.Find(Key);
					if (Value != nullptr)
					{
						auto ExpectedIt = Expected.find(Key);
						REQUIRE(ExpectedIt != Expected.end());
						REQUIRE(*Value == ExpectedIt->second);
					}
					else
					{
						REQUIRE(Expected.find(Key) == Expected.end());
					}
					break;
				}
				case 1:
				{
					ValueType Value = Map.FindOrAdd(Key);
					auto ExpectedIt = Expected.find(Key);
					if (ExpectedIt != Expected.end())
					{
						REQUIRE(Value == ExpectedIt->second);
					}
					else
					{
						REQUIRE(Value == ValueType{});
						Expected[Key] = ValueType{};
					}
					break;
				}
				case 2:
				{
					Map.Remove(Key);
					Expected.erase(Key);
					break;
				}
				case 3:
				{
					REQUIRE(Map.Contains(Key) == (Expected.count(Key) != 0));
					break;
				}
				case 4:
				{
					Map.Empty();
					Expected.clear();
					break;
				}
				case 5:
				{
					Map.Reset();
					Expected.clear();
					break;
				}
				default:
				{
					Map.Add(Key, ValueType{I});
					Expected[Key] = ValueType{I};
					break;
				}
			}
			Check(Map, Expected);
		}
	}

	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls);
	AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls = 0;
	AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls = 0;
}
