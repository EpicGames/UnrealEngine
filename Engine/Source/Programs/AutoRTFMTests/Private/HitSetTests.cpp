// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitSet.h"
#include "Catch2Includes.h"
#include <cstdint>
#include <unordered_set>
#include <random>

TEST_CASE("HitSet")
{
	struct FHasher
	{
		size_t operator()(AutoRTFM::FHitSetEntry Entry) const
		{
			return static_cast<size_t>(Entry.Payload());
		}
	};

	struct FEqualTo
	{
		bool operator()(AutoRTFM::FHitSetEntry A, AutoRTFM::FHitSetEntry B) const
		{
			return A.Payload() == B.Payload();
		}
	};

	AutoRTFM::FHitSet HitSet;
	std::unordered_set<AutoRTFM::FHitSetEntry, FHasher, FEqualTo> Expected;

	auto CheckIsExpected = [&]
	{
		REQUIRE(HitSet.GetCount() == Expected.size());
		if (Expected.size() == 0)
		{
			REQUIRE(HitSet.IsEmpty());
			REQUIRE(HitSet.begin() == HitSet.end());
		}
		else
		{
			REQUIRE(!HitSet.IsEmpty());
			REQUIRE(HitSet.begin() != HitSet.end());
		}

		REQUIRE(HitSet.GetCount() <= HitSet.GetCapacity());

		std::unordered_set<AutoRTFM::FHitSetEntry, FHasher, FEqualTo> Got;
		for (AutoRTFM::FHitSetEntry GotEntry : HitSet)
		{
			Got.insert(GotEntry);
			const bool bFoundHitSetEntryInExpected = Expected.contains(GotEntry);
			REQUIRE(bFoundHitSetEntryInExpected);
		}
		for (AutoRTFM::FHitSetEntry ExpectedEntry : Expected)
		{
			const bool bFoundExpectedInHitSet = Got.contains(ExpectedEntry);
			REQUIRE(bFoundExpectedInHitSet);
			REQUIRE(HitSet.Contains(ExpectedEntry));
		}
		REQUIRE(Got.size() == Expected.size());
	};

	auto RunTests = [&](auto Insert)
	{
		constexpr uintptr_t One = 1;
		Catch::SimplePcg32 Rand(0x1234);

		SECTION("Vary Address")
		{
			const uintptr_t Size = 0x100;
			const uintptr_t Flags = 0;
			for (size_t I = 1; I < 1000; I++)
			{
				const uintptr_t Address = I * 0x1000;
				Insert({Address, Size, Flags});
			}
			CheckIsExpected();
			for (size_t I = 1; I < 2000; I++)
			{
				const uintptr_t Address = I * 0x500;
				Insert({Address, Size, Flags});
			}
			CheckIsExpected();
		}

		SECTION("Vary Size")
		{
			const uintptr_t Address = 0x1000;
			const uintptr_t Flags = 0;
			for (size_t I = 0; I < 1000; I++)
			{
				const uintptr_t Size = I;
				Insert({Address, I, Flags});
			}
			CheckIsExpected();
			for (size_t I = 0; I < 1000; I++)
			{
				const uintptr_t Size = I;
				Insert({Address, I, Flags});
			}
			CheckIsExpected();
		}

		SECTION("Vary Flags")
		{
			const uintptr_t Address = 0x1000;
			const uintptr_t Size = 0x100;
			for (size_t I = 0; I < 15; I++)
			{
				const uintptr_t Flags = I;
				Insert({Address, Size, Flags});
			}
			CheckIsExpected();
			for (size_t I = 0; I < 15; I++)
			{
				const uintptr_t Flags = I;
				Insert({Address, Size, Flags});
			}
			CheckIsExpected();
		}

		SECTION("Vary All")
		{
			for (size_t I = 0; I < 10000; I++)
			{
				const uintptr_t Address = One << (Rand() % AutoRTFM::FHitSetEntry::AddressBits);
				const uintptr_t Size = One << (Rand() % AutoRTFM::FHitSetEntry::SizeBits);
				const uintptr_t Flags = Rand() & ((One << AutoRTFM::FHitSetEntry::FlagsBits) - 1);
				Insert({Address, Size, Flags});
			}
			CheckIsExpected();
		}

		{
			for (size_t I = 0; I < 10000; I++)
			{
				const uintptr_t Address = One << (Rand() % AutoRTFM::FHitSetEntry::AddressBits);
				const uintptr_t Size = One << (Rand() % AutoRTFM::FHitSetEntry::SizeBits);
				const uintptr_t Flags = Rand() & ((One << AutoRTFM::FHitSetEntry::FlagsBits) - 1);
				const AutoRTFM::FHitSetEntry Entry{Address, Size, Flags};
				REQUIRE(HitSet.Contains(Entry) == Expected.contains(Entry));
			}
		}
	};

	SECTION("FindOrTryInsert")
	{
		auto Insert = [&](AutoRTFM::FHitSetEntry Entry)
		{
			// We're not testing for max-capacity behavior here.
			REQUIRE(HitSet.GetCapacity() < AutoRTFM::FHitSet::MaxCapacity);

			const bool bAlreadyAdded = Expected.contains(Entry);
			
			if (HitSet.FindOrTryInsert(Entry))
			{
				REQUIRE(bAlreadyAdded);
			}
			else
			{
				REQUIRE(!bAlreadyAdded);
				Expected.insert(Entry);
			}
		};

		RunTests(Insert);
	}

	SECTION("FindOrTryInsertNoResize")
	{
		auto InsertNoResize = [&](AutoRTFM::FHitSetEntry Entry)
		{
			switch (HitSet.FindOrTryInsertNoResize(Entry))
			{
				case AutoRTFM::FHitSet::EInsertResult::Inserted:
				{
					const bool bAdded = Expected.emplace(Entry).second;
					REQUIRE(bAdded);
					break;
				}
				case AutoRTFM::FHitSet::EInsertResult::NotInserted:
				{
					REQUIRE(!Expected.contains(Entry));
					break;
				}
				case AutoRTFM::FHitSet::EInsertResult::Exists:
				{
					REQUIRE(Expected.contains(Entry));
					break;
				}
			}
		};

		RunTests(InsertNoResize);
	}
}
