// Copyright Epic Games, Inc. All Rights Reserved.

// These tests rely on us reaching into core using private
// header includes which is sometimes not so fine.
#if !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)

#include "API.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "CoreTypes.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/CallstackTracePrivate.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

UE_TRACE_CHANNEL_DEFINE(AutoRTFMInsightsChannel, "AutoRTFM transaction testing and profiling.")

struct AutoRTFMInsightsDeferer final
{
	explicit AutoRTFMInsightsDeferer(UE::Trace::FChannel& InChannel) : Channel(InChannel), bWasEnabled(InChannel)
	{
		Channel.Toggle(true);
	}

	~AutoRTFMInsightsDeferer()
	{
		Channel.Toggle(bWasEnabled);
	}

private:
	UE::Trace::FChannel& Channel;
	const bool bWasEnabled;
};

UE_TRACE_EVENT_BEGIN(Cpu, SomeTraceEvent)
	UE_TRACE_EVENT_FIELD(int32, Foo)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, SomeNoSyncTraceEvent, NoSync)
	UE_TRACE_EVENT_FIELD(int32, Foo)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, SomeImportantTraceEvent, NoSync | Important)
	UE_TRACE_EVENT_FIELD(int32, Foo)
UE_TRACE_EVENT_END()

TEST_CASE("Insights")
{
#if CPUPROFILERTRACE_ENABLED
	AutoRTFMInsightsDeferer Cpu(CpuChannel);
	REQUIRE(UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel));
#endif
	
	AutoRTFMInsightsDeferer AutoRTFMInsights(AutoRTFMInsightsChannel);
	REQUIRE(UE_TRACE_CHANNELEXPR_IS_ENABLED(AutoRTFMInsightsChannel));

	SECTION("TRACE_CPUPROFILER_EVENT_DECLARE")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_DECLARE(SomeEvent);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_DECLARE(SomeEvent);
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_USE")
	{
		TRACE_CPUPROFILER_EVENT_DECLARE(SomeEvent);
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_USE(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, true);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_USE")
	{
		TRACE_CPUPROFILER_EVENT_DECLARE(SomeEvent);
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, AutoRTFMInsightsChannel, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, AutoRTFMInsightsChannel, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, AutoRTFMInsightsChannel, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(SomeEvent, ANSI_TO_TCHAR("Wowwee"), _, AutoRTFMInsightsChannel, true);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_STR")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR(ANSI_TO_TCHAR("Wowwee"));
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_STR(ANSI_TO_TCHAR("Wowwee"));
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR(ANSI_TO_TCHAR("Wowwee"));
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), true);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(ANSI_TO_TCHAR("Wowwee"), AutoRTFMInsightsChannel, true);
			AutoRTFM::AbortTransaction();
		});
	}
	
	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Wowwee);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(Wowwee);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Wowwee);
			AutoRTFM::AbortTransaction();
		});
	}
	
	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Wowwee, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Wowwee, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Wowwee, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Wowwee, true);
			AutoRTFM::AbortTransaction();
		});
	}
	
	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(Wowwee, AutoRTFMInsightsChannel);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(Wowwee, AutoRTFMInsightsChannel);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(Wowwee, AutoRTFMInsightsChannel);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Wowwee, AutoRTFMInsightsChannel, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Wowwee, AutoRTFMInsightsChannel, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Wowwee, AutoRTFMInsightsChannel, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Wowwee, AutoRTFMInsightsChannel, true);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_TEXT")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Name);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					FString Name("Wowwee");
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Name);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Name);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*Name, AutoRTFMInsightsChannel);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					FString Name("Wowwee");
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*Name, AutoRTFMInsightsChannel);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*Name, AutoRTFMInsightsChannel);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(*Name, AutoRTFMInsightsChannel, false);
		});

		AutoRTFM::Testing::Commit([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(*Name, AutoRTFMInsightsChannel, true);
		});

		SECTION("In a child transaction")
		{
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Testing::Commit([&]
				{
					FString Name("Wowwee");
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(*Name, AutoRTFMInsightsChannel, true);
				});
			});
		}

		AutoRTFM::Testing::Abort([&]
		{
			FString Name("Wowwee");
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(*Name, AutoRTFMInsightsChannel, true);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_FLUSH")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_FLUSH();
		});

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_FLUSH();
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED();
		});

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED();
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("TRACE_CPUPROFILER_EVENT_MANUAL_START + TRACE_CPUPROFILER_EVENT_MANUAL_END")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TRACE_CPUPROFILER_EVENT_MANUAL_START(TEXT("Wowwee"));
			TRACE_CPUPROFILER_EVENT_MANUAL_END();
		});

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_MANUAL_START(TEXT("Wowwee"));
			TRACE_CPUPROFILER_EVENT_MANUAL_END();
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Abort([&]
		{
			TRACE_CPUPROFILER_EVENT_MANUAL_START(TEXT("Wowwee"));
			AutoRTFM::AbortTransaction();
		});
	}
	
#if CPUPROFILERTRACE_ENABLED
	// Some number over 10000 to force allocations.
	constexpr unsigned Iterations = 16384;
	SECTION("UE_TRACE_LOG_SCOPED_T")
	{
		AutoRTFM::Testing::Abort([&]
		{
			UE_TRACE_LOG_SCOPED_T(Cpu, SomeTraceEvent, CpuChannel) << SomeTraceEvent.Foo(42);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			for (int Index = 0; Index < Iterations; Index++)
			{
				UE_TRACE_LOG_SCOPED_T(Cpu, SomeTraceEvent, CpuChannel) << SomeTraceEvent.Foo(42);
			}
		});

		AutoRTFM::Testing::Abort([&]
		{
			UE_TRACE_LOG_SCOPED_T(Cpu, SomeNoSyncTraceEvent, CpuChannel) << SomeNoSyncTraceEvent.Foo(42);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			for (int Index = 0; Index < Iterations; Index++)
			{
				UE_TRACE_LOG_SCOPED_T(Cpu, SomeNoSyncTraceEvent, CpuChannel) << SomeNoSyncTraceEvent.Foo(42);
			}
		});
	}

	SECTION("UE_TRACE_LOG")
	{
		AutoRTFM::Testing::Abort([&]
		{
			UE_TRACE_LOG(Cpu, SomeImportantTraceEvent, CpuChannel) << SomeImportantTraceEvent.Foo(42);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			for (int Index = 0; Index < Iterations; Index++)
			{
				UE_TRACE_LOG(Cpu, SomeImportantTraceEvent, CpuChannel) << SomeImportantTraceEvent.Foo(42);
			}
		});
	}
#endif

#if UE_CALLSTACK_TRACE_ENABLED
	SECTION("CallstackTrace")
	{
		AutoRTFMInsightsDeferer Callstack(CallstackChannel);
		REQUIRE(UE_TRACE_CHANNELEXPR_IS_ENABLED(CallstackChannel));

		const int OutwithTransactionId = CallstackTrace_GetCurrentId();
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(OutwithTransactionId != CallstackTrace_GetCurrentId());
		});
	}
#endif
}

#endif // !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)
