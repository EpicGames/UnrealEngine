// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interval.h"
#include "Catch2Includes.h"

#include <cstdint>
#include <limits>
#include <vector>

using FInterval = AutoRTFM::FInterval;

static constexpr FInterval MaxInterval{static_cast<uintptr_t>(0), std::numeric_limits<uintptr_t>::max()};

struct FTreeIntervals
{
	// Intervals that are not in the tree
	std::vector<FInterval> Free;
	// Intervals that are in the tree
	std::vector<FInterval> Used;

	// Sorts the Free and Used intervals into ascending order, and checks that
	// each interval in the vector has a non-zero size and does not overlap any
	// other interval in the same vector.
	void SortAndValidate()
	{
		auto SortAndValidateIntervals = [](std::vector<FInterval>& Intervals)
		{
			std::sort(Intervals.begin(), Intervals.end());
			uintptr_t End = 0;
			for (FInterval& Interval : Intervals)
			{
				REQUIRE(Interval.Size() > 0);
				REQUIRE(End <= Interval.Start);
				End = Interval.End;
			}
		};
		SortAndValidateIntervals(Free);
		SortAndValidateIntervals(Used);
	}
};

// Populates a tree with at most MaxUsed random intervals across the entire uintptr_t interval range.
static FTreeIntervals BuildIntervals(int Seed, size_t MaxUsed)
{
	Catch::SimplePcg32 Rand(Seed);
	std::vector<FInterval> Free{MaxInterval};
	std::vector<FInterval> Used;

	while (!Free.empty() && Used.size() < MaxUsed)
	{
		// Pop a random free interval
		std::swap(Free[Rand() % Free.size()], Free.back());
		const FInterval FreeInterval = Free.back();
		Free.pop_back();

		// Steal the part of the interval
		static constexpr size_t MaxIntervalSize = 0x100000;
		const uintptr_t Size = FreeInterval.Size() > 1 ? (1 + Rand() % std::min(MaxIntervalSize, FreeInterval.Size() - 1)) : 1;
		uintptr_t Offset = FreeInterval.Size() > Size ? (Rand() % (FreeInterval.Size() - Size)) : 0;
		if (Rand() % 100 < 5) // 5% of the intervals are aligned to the start or end of another interval
		{
			Offset = (Rand() & 1) ? 0 : FreeInterval.Size() - Size;
		}

		REQUIRE(Offset + Size <= FreeInterval.Size());

		const FInterval UsedInterval{FreeInterval.Start + Offset, FreeInterval.Start + Offset + Size};

		if (FreeInterval.Start != UsedInterval.Start)
		{
			Free.push_back(FInterval{FreeInterval.Start, UsedInterval.Start});
		}
		if (UsedInterval.End != FreeInterval.End)
		{
			Free.push_back(FInterval{UsedInterval.End, FreeInterval.End});
		}

		Used.push_back(UsedInterval);
	}

	return FTreeIntervals{Free, Used};
}
