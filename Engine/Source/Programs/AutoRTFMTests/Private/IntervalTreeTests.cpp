// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "IntervalTestUtils.h"
#include "IntervalTree.h"

template <typename T> using TIntervalTree = AutoRTFM::TIntervalTree<T>;

struct FIntervalAndData
{
	FInterval Interval;
	int Data;
};

TEST_CASE("IntervalTree")
{
	AutoRTFM::UnreachableIfClosed();

	// Convenience lambda to convert a uintptr_t to a void*
	auto Ptr = [](uintptr_t Address) { return reinterpret_cast<void*>(Address); };

	auto Insert = [Ptr](TIntervalTree<int>& Tree, uintptr_t Address, uintptr_t Size, int Data)
	{
		Tree.Insert(Ptr(Address), Size, Data);
	};

	// Returns true if Overlaps() finds at least one overlapping interval for [Address, Address+Size).
	auto HasOverlap = [Ptr](const TIntervalTree<int>& Tree, uintptr_t Address, uintptr_t Size) -> bool
	{
		return Tree.Overlaps(Ptr(Address), Size);
	};

	// Checks that the interval tree has all of the expected intervals, and that Get() returns
	// the correct value for various intervals.
	auto Validate = [&] AUTORTFM_DISABLE (TIntervalTree<int>& IntervalTree, std::vector<FIntervalAndData> Expected)
	{
		std::vector<FIntervalAndData> Got;
		for (TIntervalTree<int>::FIntervalAndData Entry : IntervalTree)
		{
			Got.push_back(FIntervalAndData{Entry.Interval, Entry.Data});
		}

		REQUIRE(Expected.empty() == IntervalTree.IsEmpty());
		for (size_t I = 0, N = std::min(Expected.size(), Got.size()); I < N; I++)
		{
			REQUIRE(Expected[I].Interval.Start == Got[I].Interval.Start);
			REQUIRE(Expected[I].Interval.End == Got[I].Interval.End);
			REQUIRE(Expected[I].Data == Got[I].Data);
		}
		REQUIRE(Expected.size() == Got.size());
		for (size_t Index = 0; Index < Expected.size(); Index++)
		{
			const FIntervalAndData Entry = Expected[Index];
			const bool bTouchingPrev = Index > 0 && Expected[Index - 1].Interval.End == Entry.Interval.Start;
			const bool bTouchingNext = Index < (Expected.size() - 1) && Expected[Index + 1].Interval.Start == Entry.Interval.End;
			REQUIRE(HasOverlap(IntervalTree, Entry.Interval.Start, 1));
			REQUIRE(HasOverlap(IntervalTree, Entry.Interval.Start, Entry.Interval.Size()));
			REQUIRE((*IntervalTree.Get(Ptr(Entry.Interval.Start), Entry.Interval.Size()).begin()).Data == Entry.Data);
			REQUIRE(bTouchingPrev == HasOverlap(IntervalTree, Entry.Interval.Start - 1, 1));
			REQUIRE(bTouchingNext == HasOverlap(IntervalTree, Entry.Interval.End, 1));
		}
	};

	TIntervalTree<int> IntervalTree;

	Validate(IntervalTree, {});

	SECTION("Insert")
	{
		IntervalTree.Reset();
		Validate(IntervalTree, {});

		// New non-overlapping intervals
		Insert(IntervalTree, 40, 10, 1);
		Validate(IntervalTree, {{{40, 50}, 1}});

		Insert(IntervalTree, 20, 10, 2);
		Validate(IntervalTree, {{{20, 30}, 2}, {{40, 50}, 1}});

		Insert(IntervalTree, 60, 10, 3);
		Validate(IntervalTree, {{{20, 30}, 2}, {{40, 50}, 1}, {{60, 70}, 3}});

		Insert(IntervalTree, 80, 10, 4);
		Validate(IntervalTree, {{{20, 30}, 2}, {{40, 50}, 1}, {{60, 70}, 3}, {{80, 90}, 4}});

		// Re-inserting with different data replaces the data of the existing interval.
		Insert(IntervalTree, 20, 10, 5);
		Validate(IntervalTree, {{{20, 30}, 5}, {{40, 50}, 1}, {{60, 70}, 3}, {{80, 90}, 4}});

		Insert(IntervalTree, 40, 10, 6);
		Validate(IntervalTree, {{{20, 30}, 5}, {{40, 50}, 6}, {{60, 70}, 3}, {{80, 90}, 4}});

		Insert(IntervalTree, 60, 10, 7);
		Validate(IntervalTree, {{{20, 30}, 5}, {{40, 50}, 6}, {{60, 70}, 7}, {{80, 90}, 4}});

		Insert(IntervalTree, 80, 10, 8);
		Validate(IntervalTree, {{{20, 30}, 5}, {{40, 50}, 6}, {{60, 70}, 7}, {{80, 90}, 8}});

		// Re-inserting with same data is effectively a no-op.
		Insert(IntervalTree, 20, 10, 5);
		Validate(IntervalTree, {{{20, 30}, 5}, {{40, 50}, 6}, {{60, 70}, 7}, {{80, 90}, 8}});

		// Inserting adjacent intervals with different data: intervals stay separate (no merge).
		Insert(IntervalTree, 15, 5, 9);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{40, 50}, 6}, {{60, 70}, 7}, {{80, 90}, 8}});

		Insert(IntervalTree, 30, 5, 10);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{40, 50}, 6}, {{60, 70}, 7}, {{80, 90}, 8}});

		Insert(IntervalTree, 55, 5, 11);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{40, 50}, 6}, {{55, 60}, 11}, {{60, 70}, 7}, {{80, 90}, 8}});

		Insert(IntervalTree, 70, 5, 12);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{40, 50}, 6}, {{55, 60}, 11}, {{60, 70}, 7}, {{70, 75}, 12}, {{80, 90}, 8}});

		// Inserting between adjacent different-data intervals: new interval stays separate from both neighbours.
		Insert(IntervalTree, 35, 5, 13);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{35, 40}, 13}, {{40, 50}, 6}, {{55, 60}, 11}, {{60, 70}, 7}, {{70, 75}, 12}, {{80, 90}, 8}});

		Insert(IntervalTree, 50, 5, 14);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{35, 40}, 13}, {{40, 50}, 6}, {{50, 55}, 14}, {{55, 60}, 11}, {{60, 70}, 7}, {{70, 75}, 12}, {{80, 90}, 8}});

		Insert(IntervalTree, 75, 5, 15);
		Validate(IntervalTree, {{{15, 20}, 9}, {{20, 30}, 5}, {{30, 35}, 10}, {{35, 40}, 13}, {{40, 50}, 6}, {{50, 55}, 14}, {{55, 60}, 11}, {{60, 70}, 7}, {{70, 75}, 12}, {{75, 80}, 15}, {{80, 90}, 8}});
	}

	SECTION("Insert.SameData")
	{
		// Exact match, same data — effectively a no-op.
		Insert(IntervalTree, 40, 10, 1);
		Insert(IntervalTree, 40, 10, 1);
		Validate(IntervalTree, {{{40, 50}, 1}});

		IntervalTree.Reset();

		// New interval is a strict subset of an existing interval with the same data.
		// The existing interval absorbs the new one (no change).
		Insert(IntervalTree, 30, 30, 1); // [30,60)
		Insert(IntervalTree, 40, 10, 1); // [40,50) subset
		Validate(IntervalTree, {{{30, 60}, 1}});

		IntervalTree.Reset();

		// New interval is a strict superset — absorbs the existing interval.
		Insert(IntervalTree, 40, 10, 1); // [40,50)
		Insert(IntervalTree, 30, 30, 1); // [30,60) superset
		Validate(IntervalTree, {{{30, 60}, 1}});

		IntervalTree.Reset();

		// Left-adjacent, same data — two intervals merge into one.
		Insert(IntervalTree, 30, 10, 1); // [30,40)
		Insert(IntervalTree, 40, 10, 1); // [40,50) adjacent on right
		Validate(IntervalTree, {{{30, 50}, 1}});

		IntervalTree.Reset();

		// Right-adjacent, same data — two intervals merge into one.
		Insert(IntervalTree, 40, 10, 1); // [40,50)
		Insert(IntervalTree, 30, 10, 1); // [30,40) adjacent on left
		Validate(IntervalTree, {{{30, 50}, 1}});

		IntervalTree.Reset();

		// Both-adjacent, same data — three intervals merge into one.
		Insert(IntervalTree, 20, 10, 1); // [20,30)
		Insert(IntervalTree, 40, 10, 1); // [40,50)
		Insert(IntervalTree, 30, 10, 1); // [30,40) fills the gap
		Validate(IntervalTree, {{{20, 50}, 1}});

		IntervalTree.Reset();

		// Left partial overlap, same data — predecessor extends into the new interval; merge left.
		Insert(IntervalTree, 30, 15, 1); // [30,45)
		Insert(IntervalTree, 40, 10, 1); // [40,50) overlaps right side of existing
		Validate(IntervalTree, {{{30, 50}, 1}});

		IntervalTree.Reset();

		// Right partial overlap, same data — new interval extends into a successor; merge right.
		Insert(IntervalTree, 45, 15, 1); // [45,60)
		Insert(IntervalTree, 40, 10, 1); // [40,50) overlaps left side of existing
		Validate(IntervalTree, {{{40, 60}, 1}});

		IntervalTree.Reset();

		// Many non-adjacent same-data intervals + a spanning insert — all globbed into one.
		Insert(IntervalTree, 10, 10, 1); // [10,20)
		Insert(IntervalTree, 30, 10, 1); // [30,40)
		Insert(IntervalTree, 50, 10, 1); // [50,60)
		Insert(IntervalTree, 70, 10, 1); // [70,80)
		Insert(IntervalTree, 90, 10, 1); // [90,100)
		Validate(IntervalTree, {{{10, 20}, 1}, {{30, 40}, 1}, {{50, 60}, 1}, {{70, 80}, 1}, {{90, 100}, 1}});
		Insert(IntervalTree, 5, 100, 1);  // [5,105) spans all five
		Validate(IntervalTree, {{{5, 105}, 1}});
	}

	SECTION("Insert.DifferentData")
	{
		// Exact match, different data — data is replaced.
		Insert(IntervalTree, 40, 10, 1);
		Insert(IntervalTree, 40, 10, 2);
		Validate(IntervalTree, {{{40, 50}, 2}});

		IntervalTree.Reset();

		// New interval is a strict subset, different data — existing is split into three.
		Insert(IntervalTree, 30, 30, 1); // [30,60)
		Insert(IntervalTree, 40, 10, 2); // [40,50) subset with different data
		Validate(IntervalTree, {{{30, 40}, 1}, {{40, 50}, 2}, {{50, 60}, 1}});

		IntervalTree.Reset();

		// New interval is a strict superset, different data — existing is deleted.
		Insert(IntervalTree, 40, 10, 1); // [40,50)
		Insert(IntervalTree, 30, 30, 2); // [30,60) superset with different data
		Validate(IntervalTree, {{{30, 60}, 2}});

		IntervalTree.Reset();

		// Left partial overlap, different data — existing end is trimmed.
		Insert(IntervalTree, 30, 15, 1); // [30,45)
		Insert(IntervalTree, 40, 10, 2); // [40,50) overlaps right side of existing
		Validate(IntervalTree, {{{30, 40}, 1}, {{40, 50}, 2}});

		IntervalTree.Reset();

		// Right partial overlap, different data — existing start is trimmed (via remove+re-insert).
		Insert(IntervalTree, 45, 15, 1); // [45,60)
		Insert(IntervalTree, 40, 10, 2); // [40,50) overlaps left side of existing
		Validate(IntervalTree, {{{40, 50}, 2}, {{50, 60}, 1}});

		IntervalTree.Reset();

		// Adjacent left, different data — intervals stay separate (no merge).
		Insert(IntervalTree, 30, 10, 1); // [30,40)
		Insert(IntervalTree, 40, 10, 2); // [40,50)
		Validate(IntervalTree, {{{30, 40}, 1}, {{40, 50}, 2}});

		IntervalTree.Reset();

		// Adjacent right, different data — intervals stay separate (no merge).
		Insert(IntervalTree, 50, 10, 1); // [50,60)
		Insert(IntervalTree, 40, 10, 2); // [40,50)
		Validate(IntervalTree, {{{40, 50}, 2}, {{50, 60}, 1}});
	}

	SECTION("Insert.GlobMany")
	{
		// Insert a span with the same data as all existing intervals — everything merges into one.
		Insert(IntervalTree, 10, 10, 1); // [10,20)
		Insert(IntervalTree, 30, 10, 1); // [30,40)
		Insert(IntervalTree, 50, 10, 1); // [50,60)
		Insert(IntervalTree, 70, 10, 1); // [70,80)
		Insert(IntervalTree, 90, 10, 1); // [90,100)
		Insert(IntervalTree, 5, 100, 1); // [5,105) with same data
		Validate(IntervalTree, {{{5, 105}, 1}});

		IntervalTree.Reset();

		// Insert a span that mixes same-data and different-data neighbours.
		// Same-data intervals are merged in; different-data intervals within the range are deleted.
		// The effective span expands to absorb all touching same-data intervals.
		Insert(IntervalTree, 10, 10, 1); // [10,20) — same data as the spanning insert
		Insert(IntervalTree, 30, 10, 2); // [30,40) — different data
		Insert(IntervalTree, 50, 10, 1); // [50,60) — same data
		Insert(IntervalTree, 70, 10, 2); // [70,80) — different data
		Insert(IntervalTree, 90, 10, 1); // [90,100) — same data
		// Insert [15,95) with data=1. Its effective range expands:
		//   - [10,20),1 predecessor overlaps from left -> absorbed -> EffStart=10
		//   - [30,40),2 fully contained, different data -> deleted
		//   - [50,60),1 same data -> absorbed (EffEnd stays 95)
		//   - [70,80),2 fully contained, different data -> deleted
		//   - [90,100),1 overlaps past EffEnd=95, same data -> absorbed -> EffEnd=100
		// Result: one interval [10,100) with data=1.
		Insert(IntervalTree, 15, 80, 1); // [15,95)
		Validate(IntervalTree, {{{10, 100}, 1}});
	}

	SECTION("Insert.StressRemove")
	{
		// Insert many non-adjacent same-data intervals, then insert a span covering
		// them all. This exercises Remove() with potential two-children swap during
		// Phase B's forward scan, and verifies the LowerBound re-query approach.
		for (int I = 0; I < 100; I++)
		{
			Insert(IntervalTree, 100 + static_cast<uintptr_t>(I) * 20, 10, 1);
		}
		Insert(IntervalTree, 100, 1990, 1); // [100, 2090) spans all 100
		Validate(IntervalTree, {{{100, 2090}, 1}});

		IntervalTree.Reset();

		// Same but with different data values to test fully-contained deletion
		// of many nodes during a single Insert call.
		for (int I = 0; I < 100; I++)
		{
			Insert(IntervalTree, 100 + static_cast<uintptr_t>(I) * 20, 10, I);
		}
		Insert(IntervalTree, 100, 1990, 999); // [100, 2090) replaces all 100
		Validate(IntervalTree, {{{100, 2090}, 999}});
	}

	SECTION("Merge")
	{
		Insert(IntervalTree, 20, 10, 1);
		Insert(IntervalTree, 40, 10, 2);
		Insert(IntervalTree, 60, 10, 3);
		Insert(IntervalTree, 80, 10, 4);
		Validate(IntervalTree, {{{20, 30}, 1}, {{40, 50}, 2}, {{60, 70}, 3}, {{80, 90}, 4}});

		SECTION("Intertwined")
		{
			TIntervalTree<int> IntervalTreeB;
			Insert(IntervalTreeB, 10, 5, 10);
			Insert(IntervalTreeB, 33, 6, 11);
			Insert(IntervalTreeB, 54, 2, 12);
			Insert(IntervalTreeB, 73, 3, 13);
			Insert(IntervalTreeB, 95, 2, 14);
			Validate(IntervalTreeB, {{{10, 15}, 10}, {{33, 39}, 11}, {{54, 56}, 12}, {{73, 76}, 13}, {{95, 97}, 14}});

			IntervalTree.Merge(IntervalTreeB);
			Validate(IntervalTree, {{{10, 15}, 10}, {{20, 30}, 1}, {{33, 39}, 11}, {{40, 50}, 2}, {{54, 56}, 12}, {{60, 70}, 3}, {{73, 76}, 13}, {{80, 90}, 4}, {{95, 97}, 14}});
		}

		SECTION("Expand")
		{
			// IntervalTreeB's intervals are adjacent to but not overlapping those in IntervalTree,
			// and all have different data. After merge, all intervals remain separate — no merging occurs.
			TIntervalTree<int> IntervalTreeB;
			Insert(IntervalTreeB, 15, 5, 10);  // [15,20) adjacent-left to [20,30),1
			Insert(IntervalTreeB, 30, 3, 11);  // [30,33) adjacent-right to [20,30),1
			Insert(IntervalTreeB, 37, 3, 12);  // [37,40) adjacent-left to [40,50),2
			Insert(IntervalTreeB, 50, 2, 13);  // [50,52) adjacent-right to [40,50),2
			Insert(IntervalTreeB, 55, 5, 14);  // [55,60) adjacent-left to [60,70),3
			Insert(IntervalTreeB, 70, 4, 15);  // [70,74) adjacent-right to [60,70),3
			Insert(IntervalTreeB, 78, 2, 16);  // [78,80) adjacent-left to [80,90),4
			Insert(IntervalTreeB, 90, 3, 17);  // [90,93) adjacent-right to [80,90),4
			Validate(IntervalTreeB, {{{15, 20}, 10}, {{30, 33}, 11}, {{37, 40}, 12}, {{50, 52}, 13}, {{55, 60}, 14}, {{70, 74}, 15}, {{78, 80}, 16}, {{90, 93}, 17}});

			IntervalTree.Merge(IntervalTreeB);
			Validate(IntervalTree, {
				{{15, 20}, 10}, {{20, 30}, 1}, {{30, 33}, 11}, {{37, 40}, 12}, {{40, 50}, 2},
				{{50, 52}, 13}, {{55, 60}, 14}, {{60, 70}, 3}, {{70, 74}, 15}, {{78, 80}, 16},
				{{80, 90}, 4}, {{90, 93}, 17}
			});
		}

		SECTION("Overlap")
		{
			// IntervalTreeB's intervals exactly fill the gaps between IntervalTree's intervals,
			// touching both neighbours at their endpoints. All data values are different, so
			// no merging occurs and all intervals remain as separate entries.
			TIntervalTree<int> IntervalTreeB;
			Insert(IntervalTreeB, 30, 10, 10); // [30,40) fills gap between [20,30) and [40,50)
			Insert(IntervalTreeB, 50, 10, 11); // [50,60) fills gap between [40,50) and [60,70)
			Insert(IntervalTreeB, 70, 10, 12); // [70,80) fills gap between [60,70) and [80,90)
			Validate(IntervalTreeB, {{{30, 40}, 10}, {{50, 60}, 11}, {{70, 80}, 12}});

			IntervalTree.Merge(IntervalTreeB);
			Validate(IntervalTree, {
				{{20, 30}, 1}, {{30, 40}, 10}, {{40, 50}, 2},
				{{50, 60}, 11}, {{60, 70}, 3}, {{70, 80}, 12}, {{80, 90}, 4}
			});
		}

		SECTION("PartialOverlap")
		{
			// IntervalTreeB's intervals partially overlap IntervalTree's intervals.
			// Different data: existing intervals are trimmed to make room for the merged ones.
			// IntervalTree has: [20,30),1  [40,50),2  [60,70),3  [80,90),4
			TIntervalTree<int> IntervalTreeB;
			Insert(IntervalTreeB, 25, 10, 10); // [25,35) overlaps [20,30),1 from the right
			Insert(IntervalTreeB, 45, 10, 11); // [45,55) overlaps [40,50),2 from the right
			Insert(IntervalTreeB, 55, 10, 12); // [55,65) overlaps [60,70),3 from the left
			Insert(IntervalTreeB, 75, 20, 13); // [75,95) straddles [80,90),4 entirely
			Validate(IntervalTreeB, {{{25, 35}, 10}, {{45, 55}, 11}, {{55, 65}, 12}, {{75, 95}, 13}});

			IntervalTree.Merge(IntervalTreeB);
			// [20,30),1 trimmed to [20,25),1 by [25,35),10
			// [40,50),2 trimmed to [40,45),2 by [45,55),11
			// [60,70),3 trimmed to [65,70),3 by [55,65),12
			// [80,90),4 deleted by [75,95),13
			Validate(IntervalTree, {
				{{20, 25}, 1}, {{25, 35}, 10}, {{40, 45}, 2}, {{45, 55}, 11},
				{{55, 65}, 12}, {{65, 70}, 3}, {{75, 95}, 13}
			});
		}
	}

	SECTION("Soak")
	{
		for (int Pass = 0; Pass < 32; Pass++)
		{
			IntervalTree.Reset();
			Validate(IntervalTree, {});

			FTreeIntervals Intervals = BuildIntervals(/* Seed */ Pass, /* MaxUsed */ 1000);

			for (FInterval Interval : Intervals.Used)
			{
				Insert(IntervalTree, Interval.Start, Interval.Size(), 0);
			}

			Intervals.SortAndValidate();

			for (const FInterval& FreeInterval : Intervals.Free)
			{
				REQUIRE(!HasOverlap(IntervalTree, FreeInterval.Start, FreeInterval.Size()));
				REQUIRE(!HasOverlap(IntervalTree, FreeInterval.Start, 1));
				REQUIRE(!HasOverlap(IntervalTree, FreeInterval.End - 1, 1));
			}
			for (const FInterval& UsedInterval : Intervals.Used)
			{
				REQUIRE(HasOverlap(IntervalTree, UsedInterval.Start, UsedInterval.Size()));
				REQUIRE(HasOverlap(IntervalTree, UsedInterval.Start, 1));
				REQUIRE(HasOverlap(IntervalTree, UsedInterval.End - 1, 1));
			}
		}
	}

	SECTION("Soak.Overlapping")
	{
		// Validates the tree invariants after inserting intervals with overlaps.
		auto ValidateOrdered = [&] AUTORTFM_DISABLE (TIntervalTree<int>& Tree)
		{
			uintptr_t PrevEnd = 0;
			for (TIntervalTree<int>::FIntervalAndData Entry : Tree)
			{
				REQUIRE(Entry.Interval.Start >= PrevEnd);
				REQUIRE(Entry.Interval.End > Entry.Interval.Start);
				REQUIRE(HasOverlap(Tree, Entry.Interval.Start, Entry.Interval.Size()));
				REQUIRE((*Tree.Get(Ptr(Entry.Interval.Start), Entry.Interval.Size()).begin()).Data == Entry.Data);
				PrevEnd = Entry.Interval.End;
			}
		};

		for (int Pass = 0; Pass < 32; Pass++)
		{
			IntervalTree.Reset();
			Catch::SimplePcg32 Rand(Pass + 1000);

			// Insert random overlapping intervals with varying data.
			for (int I = 0; I < 500; I++)
			{
				uintptr_t Start = 1 + (Rand() % 1000);
				uintptr_t Size = 1 + (Rand() % 50);
				int Data = static_cast<int>(Rand() % 5);
				Insert(IntervalTree, Start, Size, Data);
			}

			ValidateOrdered(IntervalTree);
		}
	}

	SECTION("Get")
	{
		// Helper: collect Get() results into a vector for easy comparison.
		auto GetAll = [Ptr](const TIntervalTree<int>& Tree, uintptr_t Address, uintptr_t Size) -> std::vector<FIntervalAndData>
		{
			std::vector<FIntervalAndData> Results;
			for (TIntervalTree<int>::FIntervalAndData Entry : Tree.Get(Ptr(Address), Size))
			{
				Results.push_back(FIntervalAndData{Entry.Interval, Entry.Data});
			}
			return Results;
		};

		// Empty tree — empty range.
		REQUIRE(GetAll(IntervalTree, 40, 10).empty());

		// Populate: [20,30),1  [40,50),2  [60,70),3  [80,90),4
		Insert(IntervalTree, 20, 10, 1);
		Insert(IntervalTree, 40, 10, 2);
		Insert(IntervalTree, 60, 10, 3);
		Insert(IntervalTree, 80, 10, 4);

		// No overlap — query falls in a gap.
		REQUIRE(GetAll(IntervalTree, 30, 10).empty()); // [30,40)
		REQUIRE(GetAll(IntervalTree, 10, 10).empty()); // [10,20)
		REQUIRE(GetAll(IntervalTree, 90, 10).empty()); // [90,100)

		// Single exact match.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 40, 10);
			REQUIRE(Results.size() == 1);
			REQUIRE(Results[0].Interval.Start == 40);
			REQUIRE(Results[0].Interval.End == 50);
			REQUIRE(Results[0].Data == 2);
		}

		// Query starts before a stored interval but overlaps it (SOL-8992 case).
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 35, 10); // [35,45) overlaps [40,50)
			REQUIRE(Results.size() == 1);
			REQUIRE(Results[0].Interval.Start == 40);
			REQUIRE(Results[0].Data == 2);
		}

		// Query ends after a stored interval.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 45, 10); // [45,55) overlaps [40,50)
			REQUIRE(Results.size() == 1);
			REQUIRE(Results[0].Interval.Start == 40);
			REQUIRE(Results[0].Data == 2);
		}

		// Query spans multiple intervals.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 25, 50); // [25,75) overlaps [20,30), [40,50), [60,70)
			REQUIRE(Results.size() == 3);
			REQUIRE(Results[0].Interval.Start == 20);
			REQUIRE(Results[0].Data == 1);
			REQUIRE(Results[1].Interval.Start == 40);
			REQUIRE(Results[1].Data == 2);
			REQUIRE(Results[2].Interval.Start == 60);
			REQUIRE(Results[2].Data == 3);
		}

		// Query spans all intervals.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 0, 100); // [0,100) overlaps all four
			REQUIRE(Results.size() == 4);
		}

		// Query contained within a single interval.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 42, 3); // [42,45) inside [40,50)
			REQUIRE(Results.size() == 1);
			REQUIRE(Results[0].Data == 2);
		}

		// Predecessor-only overlap.
		{
			std::vector<FIntervalAndData> Results = GetAll(IntervalTree, 25, 10); // [25,35) overlaps [20,30) only
			REQUIRE(Results.size() == 1);
			REQUIRE(Results[0].Interval.Start == 20);
			REQUIRE(Results[0].Data == 1);
		}

		// Adjacent but non-overlapping (half-open intervals don't overlap at endpoints).
		REQUIRE(GetAll(IntervalTree, 30, 10).empty()); // [30,40) — between [20,30) and [40,50)
		REQUIRE(GetAll(IntervalTree, 50, 10).empty()); // [50,60) — between [40,50) and [60,70)

		// Single-byte query at interval start and end.
		REQUIRE(GetAll(IntervalTree, 40, 1).size() == 1); // first byte of [40,50)
		REQUIRE(GetAll(IntervalTree, 49, 1).size() == 1); // last byte of [40,50)
		REQUIRE(GetAll(IntervalTree, 50, 1).empty());     // one past end

		// Soak: random queries against BuildIntervals, brute-force verify.
		for (int Pass = 0; Pass < 32; Pass++)
		{
			IntervalTree.Reset();
			FTreeIntervals Intervals = BuildIntervals(/* Seed */ Pass + 2000, /* MaxUsed */ 500);

			for (FInterval Interval : Intervals.Used)
			{
				Insert(IntervalTree, Interval.Start, Interval.Size(), 0);
			}

			// Collect the tree's actual intervals (which may have been merged).
			std::vector<FInterval> TreeIntervals;
			for (TIntervalTree<int>::FIntervalAndData Entry : IntervalTree)
			{
				TreeIntervals.push_back(Entry.Interval);
			}

			Catch::SimplePcg32 Rand(Pass + 3000);
			for (int Q = 0; Q < 200; Q++)
			{
				uintptr_t QStart = Rand() % 0x200000;
				uintptr_t QSize = 1 + Rand() % 0x100000;
				uintptr_t QEnd = QStart + QSize;

				// Brute-force: count tree intervals that overlap [QStart, QEnd).
				size_t ExpectedCount = 0;
				for (const FInterval& Interval : TreeIntervals)
				{
					if (Interval.Start < QEnd && QStart < Interval.End)
					{
						ExpectedCount++;
					}
				}

				std::vector<FIntervalAndData> Results = GetAll(IntervalTree, QStart, QSize);
				REQUIRE(Results.size() == ExpectedCount);

				// Verify each result actually overlaps the query.
				for (const FIntervalAndData& Entry : Results)
				{
					REQUIRE(Entry.Interval.Start < QEnd);
					REQUIRE(QStart < Entry.Interval.End);
				}
			}
		}
	}

	IntervalTree.Reset();
	Validate(IntervalTree, {});
}
