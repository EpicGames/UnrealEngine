// Copyright Epic Games, Inc. All Rights Reserved.

#include "Catch2Includes.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"

#if PLATFORM_WINDOWS

#include "WindowsHeader.h"

TEST_CASE("LocalStorage.TLS")
{
	DWORD TlsSlot = TlsAlloc();
	int One = 1, Two = 2;

	SECTION("Commit")
	{
		TlsSetValue(TlsSlot, &One);

		AutoRTFM::Testing::Commit([&]
		{
			TlsSetValue(TlsSlot, &Two);
		});

		REQUIRE(TlsGetValue(TlsSlot) == &Two);
	}

	SECTION("Free")
	{
		TlsSetValue(TlsSlot, &One);

		AutoRTFM::Testing::Abort([&]
		{
			TlsSetValue(TlsSlot, &Two);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(TlsGetValue(TlsSlot) == &One);
	}

	TlsFree(TlsSlot);
}

TEST_CASE("LocalStorage.FLS")
{
	// This is identical to the TLS test, except `Tls` has been replaced with `Fls`.
	DWORD FlsSlot = FlsAlloc(/*lpCallback=*/ nullptr);
	int One = 1, Two = 2;

	SECTION("Commit")
	{
		FlsSetValue(FlsSlot, &One);

		AutoRTFM::Testing::Commit([&]
		{
			FlsSetValue(FlsSlot, &Two);
		});

		REQUIRE(FlsGetValue(FlsSlot) == &Two);
	}

	SECTION("Free")
	{
		FlsSetValue(FlsSlot, &One);

		AutoRTFM::Testing::Abort([&]
		{
			FlsSetValue(FlsSlot, &Two);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(FlsGetValue(FlsSlot) == &One);
	}

	FlsFree(FlsSlot);
}

#endif
