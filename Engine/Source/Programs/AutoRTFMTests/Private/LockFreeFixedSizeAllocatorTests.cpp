// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "Catch2Includes.h"
#include "AutoRTFM/Testing.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Containers/LockFreeList.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSafeCounter64.h"

TEMPLATE_TEST_CASE("LockFreeFixedSizeAllocator", "", FThreadSafeCounter, FThreadSafeCounter64)
{
	using ALockFreeFixedSizeAllocator = TLockFreeFixedSizeAllocator<128, PLATFORM_CACHE_LINE_SIZE, TestType>;
	using IntegerType = TestType::IntegerType;

	SECTION("Transact(Allocate)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		void* Blob = nullptr;

		AutoRTFM::Testing::Commit([&]
		{
			Blob = Allocator.Allocate();
		});

		REQUIRE(nullptr != Blob);

		Allocator.Free(Blob);
	}

	SECTION("Transact(Allocate, Abort)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		void* Blob = nullptr;

		AutoRTFM::Testing::Abort([&]
		{
			Blob = Allocator.Allocate();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(nullptr == Blob);

		// When we abort we'll actually return the allocated memory to the allocator!
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Transact(new(ALockFreeFixedSizeAllocator), Allocate, delete(Allocator), Abort)")
	{
		void* Blob = nullptr;

		AutoRTFM::Testing::Abort([&]
		{
			ALockFreeFixedSizeAllocator* Allocator = new ALockFreeFixedSizeAllocator();
			Blob = Allocator->Allocate();
			delete Allocator;
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(nullptr == Blob);
	}

	SECTION("Transact(Transact(Allocate, Abort), Allocate)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		void* Blob = nullptr;

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Abort([&]
			{
				Blob = Allocator.Allocate();
				AutoRTFM::AbortTransaction();
			});
			Blob = Allocator.Allocate();
		});

		REQUIRE(nullptr != Blob);

		Allocator.Free(Blob);

		// Check that the inner abort did eagerly return the allocation to the allocator,
		// and that the outer allocate reused that allocation.
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Transact(Free)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		void* Blob = Allocator.Allocate();
		IntegerType NumUsed = 0;
		IntegerType NumFree = 0;

		AutoRTFM::Testing::Commit([&]
		{
			Allocator.Free(Blob);
			NumUsed = Allocator.GetNumUsed();
			NumFree = Allocator.GetNumFree();
		});

		// Even though we freed in the transaction, we won't do the free until on-commit so
		// the query of these within the transaction will not be updated!
		REQUIRE(1 == NumUsed);
		REQUIRE(0 == NumFree);

		// But after the transaction they will return the correct values.
		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Transact(Free, Abort)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		void* Blob = Allocator.Allocate();

		AutoRTFM::Testing::Abort([&]
		{
			Allocator.Free(Blob);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(1 == Allocator.GetNumUsed());
		REQUIRE(0 == Allocator.GetNumFree());

		Allocator.Free(Blob);
	}

	SECTION("Transact(Allocate, Free)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		AutoRTFM::Testing::Commit([&]
			{
				void* Blob = Allocator.Allocate();
				Allocator.Free(Blob);
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Transact(Allocate, Free, Abort)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		AutoRTFM::Testing::Abort([&]
		{
			void* Blob = Allocator.Allocate();
			Allocator.Free(Blob);
			AutoRTFM::AbortTransaction();
		});

		// Even though we aborted, the allocation will be cached in the allocator!
		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Allocate, Allocate, Free, Free, Free, Transact(Trim)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		void* Blob0 = Allocator.Allocate();
		void* Blob1 = Allocator.Allocate();
		void* Blob2 = Allocator.Allocate();
		Allocator.Free(Blob0);
		Allocator.Free(Blob1);
		Allocator.Free(Blob2);

		REQUIRE(3 == Allocator.GetNumFree());

		AutoRTFM::Testing::Commit([&]
			{
				Allocator.Trim();
			});

		REQUIRE(0 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Allocate, Allocate, Free, Free, Free, Transact(Trim, Abort)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		void* Blob0 = Allocator.Allocate();
		void* Blob1 = Allocator.Allocate();
		void* Blob2 = Allocator.Allocate();
		Allocator.Free(Blob0);
		Allocator.Free(Blob1);
		Allocator.Free(Blob2);

		REQUIRE(3 == Allocator.GetNumFree());

		AutoRTFM::Testing::Abort([&]
			{
				Allocator.Trim();
				AutoRTFM::AbortTransaction();
			});

		// We aborted so the trim did not happen!
		REQUIRE(3 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Allocate, Allocate, Transact(Trim, Allocate, Free, Free, Free, Free)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		void* Blob0 = Allocator.Allocate();
		void* Blob1 = Allocator.Allocate();
		void* Blob2 = Allocator.Allocate();

		AutoRTFM::Testing::Commit([&]
		{
			Allocator.Trim();

			void* Blob = Allocator.Allocate();
			Allocator.Free(Blob);
			Allocator.Free(Blob0);
			Allocator.Free(Blob1);
			Allocator.Free(Blob2);
		});

		REQUIRE(4 == Allocator.GetNumFree());
	}
	
	SECTION("Transact(GetNumFree), Allocate, Free, Transact(GetNumFree)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		IntegerType NumFree = 0;

		AutoRTFM::Testing::Commit([&]
		{
			NumFree = Allocator.GetNumFree();
		});

		REQUIRE(0 == NumFree);

		void* const Blob = Allocator.Allocate();
		Allocator.Free(Blob);

		AutoRTFM::Testing::Commit([&]
		{
			NumFree = Allocator.GetNumFree();
		});

		REQUIRE(1 == NumFree);
	}

	SECTION("Transact(GetNumUsed), Allocate, Transact(GetNumUsed), Free")
	{
		ALockFreeFixedSizeAllocator Allocator;
		IntegerType NumUsed = 0;

		AutoRTFM::Testing::Commit([&]
		{
			NumUsed = Allocator.GetNumUsed();
		});

		REQUIRE(0 == NumUsed);

		void* const Blob = Allocator.Allocate();

		AutoRTFM::Testing::Commit([&]
		{
			NumUsed = Allocator.GetNumUsed();
		});

		REQUIRE(1 == NumUsed);

		Allocator.Free(Blob);
	}

	SECTION("Transact(Open(Allocate, Free), Allocate, Free)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Open([&]
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				});

				void* Blob = Allocator.Allocate();
				Allocator.Free(Blob);
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Transact(Allocate, Free, Open(Allocate, Free))")
	{
		ALockFreeFixedSizeAllocator Allocator;

		AutoRTFM::Testing::Commit([&]
			{
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				}
				AutoRTFM::Open([&]
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				});
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(2 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Free, Transact(Open(Allocate, Free), Allocate, Free)")
	{
		ALockFreeFixedSizeAllocator Allocator;
		{
			void* Blob = Allocator.Allocate();
			Allocator.Free(Blob);
		}
		AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Open([&]
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				});

				void* Blob = Allocator.Allocate();
				Allocator.Free(Blob);
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}

	SECTION("Allocate, Free, Transact(Allocate, Free, Open(Allocate, Free))")
	{
		ALockFreeFixedSizeAllocator Allocator;
		{
			void* Blob = Allocator.Allocate();
			Allocator.Free(Blob);
		}
		AutoRTFM::Testing::Commit([&]
			{
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				}
				AutoRTFM::Open([&]
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				});
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(2 == Allocator.GetNumFree());
	}

	// Specific test for SOL-7378
	SECTION("Transact(TLockFreePointerListFIFO::Ctor, Open(Allocate, Free), TLockFreePointerListFIFO::Dtor)")
	{
		ALockFreeFixedSizeAllocator Allocator;

		AutoRTFM::Testing::Commit([&]
			{
				TLockFreePointerListFIFO<int, 64> FIFO;
				
				AutoRTFM::Open([&]
				{
					void* Blob = Allocator.Allocate();
					Allocator.Free(Blob);
				});
			});

		REQUIRE(0 == Allocator.GetNumUsed());
		REQUIRE(1 == Allocator.GetNumFree());
	}
}
