// Copyright Epic Games, Inc. All Rights Reserved.

// These tests rely on us reaching into AutoRTFM using private
// header includes which is sometimes not so fine.
#if !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)

#include "LongJump.h"
#include "Catch2Includes.h"

TEST_CASE("LongJump")
{
	struct FSetOnDestruct
	{
		bool& bDestructed;
		FSetOnDestruct(bool& bDestructed) : bDestructed{bDestructed} {}
		~FSetOnDestruct() { bDestructed = true; }
	};
	SECTION("NoThrow")
	{
		bool bTryCalled = false;
		bool bCatchCalled = false;

		AutoRTFM::FLongJump LongJump;
		LongJump.TryCatch(
		/* Try */ [&]
		{
			bTryCalled = true;
		},
		/* Catch */ [&]
		{
			bCatchCalled = true;
		});
		
		REQUIRE(bTryCalled);
		REQUIRE(!bCatchCalled);
	}
	SECTION("Throw")
	{
		bool bPostThrow = false;
		bool bCatchCalled = false;

		AutoRTFM::FLongJump LongJump;
		LongJump.TryCatch(
		/* Try */ [&]
		{
			LongJump.Throw();
			PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
			bPostThrow = true;
			PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
		},
		/* Catch */ [&]
		{
			bCatchCalled = true;
		});
		
		REQUIRE(!bPostThrow);
		REQUIRE(bCatchCalled);
	}
	SECTION("NoUnwind")
	{
		SECTION("Direct")
		{
			bool bDtorCalled = false;

			AutoRTFM::FLongJump LongJump;
			LongJump.TryCatch(
			/* Try */ [&]
			{
				FSetOnDestruct SetOnDestruct{bDtorCalled};
				LongJump.Throw();
			},
			/* Catch */ [&]
			{});
			
			REQUIRE(!bDtorCalled);
		}
		SECTION("Nested")
		{
			bool bDtorCalled = false;

			AutoRTFM::FLongJump LongJump;
			LongJump.TryCatch(
			/* Try */ [&]
			{
				struct FNestedCalls
				{
					UE_AUTORTFM_FORCENOINLINE void F3(AutoRTFM::FLongJump& LongJump)
					{
						LongJump.Throw();
					}
					UE_AUTORTFM_FORCENOINLINE void F2(AutoRTFM::FLongJump& LongJump)
					{
						F3(LongJump);
					}
					UE_AUTORTFM_FORCENOINLINE void F1(AutoRTFM::FLongJump& LongJump)
					{
						F2(LongJump);
					}
				};
				FSetOnDestruct SetOnDestruct{bDtorCalled};
				FNestedCalls{}.F1(LongJump);
			},
			/* Catch */ [&]
			{
			});
			
			REQUIRE(!bDtorCalled);
		}
	}
}

TEST_CASE("LongJump.Benchmarks")
{
	BENCHMARK("NoThrow", InputValue)
	{
		int DoNotOptimizeAway = 0;
		for (int I = 0; I < 1024; I++)
		{
			AutoRTFM::FLongJump LongJump;
			LongJump.TryCatch(
			/* Try */ [&]
			{
				DoNotOptimizeAway += InputValue;
			},
			/* Catch */ [&]
			{
			});
		}
		return DoNotOptimizeAway;
	};

	BENCHMARK("Throw", InputValue)
	{
		int DoNotOptimizeAway = 0;
		for (int I = 0; I < 1024; I++)
		{
			AutoRTFM::FLongJump LongJump;
			LongJump.TryCatch(
			/* Try */ [&]
			{
				LongJump.Throw();
			},
			/* Catch */ [&]
			{
				DoNotOptimizeAway += InputValue;
			});
		}
		return DoNotOptimizeAway;
	};
}

#endif // !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)
