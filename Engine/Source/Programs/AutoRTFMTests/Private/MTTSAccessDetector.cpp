// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"

#include <atomic>
#include <thread>

#if ENABLE_MT_DETECTOR

TEMPLATE_TEST_CASE("MTTransactionallySafeAccessDetector", "[MTAccessDetector]", FRWTransactionallySafeAccessDetector)
{
	SECTION("Transact(UE_MT_SCOPED_READ_ACCESS, Scope(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("UE_MT_SCOPED_READ_ACCESS, Transact(UE_MT_SCOPED_READ_ACCESS)")
	{
		TestType Detector;
		UE_MT_SCOPED_READ_ACCESS(Detector);
		AutoRTFM::Testing::Commit([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);
		});
	}

	SECTION("UE_MT_SCOPED_READ_ACCESS, Transact(UE_MT_SCOPED_READ_ACCESS, Abort)")
	{
		TestType Detector;
		UE_MT_SCOPED_READ_ACCESS(Detector);
		AutoRTFM::Testing::Abort([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(UE_MT_SCOPED_WRITE_ACCESS)")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
		});
	}

	SECTION("Transact(UE_MT_SCOPED_WRITE_ACCESS, Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			}
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			}
		});
	}

	SECTION("Transact(UE_MT_SCOPED_READ_ACCESS, Open(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(UE_MT_SCOPED_READ_ACCESS, Open(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS, Open(UE_MT_SCOPED_READ_ACCESS)), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
				AutoRTFM::Open([&]
				{
					UE_MT_SCOPED_READ_ACCESS(Detector);
				});
			}
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Open(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Open(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Open(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Open(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Open(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
			UE_MT_SCOPED_READ_ACCESS(Detector);
		});
	}

	SECTION("Transact(Open(UE_MT_SCOPED_READ_ACCESS), Scope(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
		});
	}

	SECTION("Transact(Open(UE_MT_SCOPED_WRITE_ACCESS), Scope(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
			UE_MT_SCOPED_READ_ACCESS(Detector);
		});
	}

	SECTION("Transact(Open(UE_MT_SCOPED_WRITE_ACCESS), UE_MT_SCOPED_WRITE_ACCESS)")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS, Spawn(UE_MT_SCOPED_READ_ACCESS)))")
	{
		TestType Detector;
		std::atomic_uint Handshake = 0; // Used to order this thread and the spawnee.
		std::thread OtherThread;
		AutoRTFM::Testing::Commit([&]
		{
			// Because we can retry transactions reset handshake to zero.
			AutoRTFM::Open([&]
			{
				Handshake = 0;
			});

			{
				UE_MT_SCOPED_READ_ACCESS(Detector);

				AutoRTFM::Open([&]
				{
					OtherThread = std::thread([&] AUTORTFM_ENABLE // SOL-8741
						{
							// Let the main thread progress.
							Handshake += 1;

							{
								UE_MT_SCOPED_READ_ACCESS(Detector);

								// Wait for the main thread.
								while (2 != Handshake) {}
							}
						});
				});
			}

			AutoRTFM::Open([&]
			{
				// Wait for the spawnee.
				while (1 != Handshake) {}

				// Let the spawnee progress.
				Handshake += 1;

				// Wait for the spawnee to finish before unwinding the
				// stack and invalidating the Handshake and Detector
				// references used by the thread.
				OtherThread.join();
			});
		});
	}

	SECTION("Transact(UE_MT_SCOPED_READ_ACCESS, Transact(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			UE_MT_SCOPED_READ_ACCESS(Detector);

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Commit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Abort([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::Testing::Abort([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_READ_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Abort([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), Transact(UE_MT_SCOPED_WRITE_ACCESS, Abort))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::Testing::Abort([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), OnCommit(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::OnCommit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), OnCommit(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::OnCommit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), OnCommit(UE_MT_SCOPED_READ_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::OnCommit([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), OnCommit(UE_MT_SCOPED_WRITE_ACCESS))")
	{
		TestType Detector;
		AutoRTFM::Testing::Commit([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::OnCommit([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), OnAbort(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::OnAbort([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_READ_ACCESS), OnAbort(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			}

			AutoRTFM::OnAbort([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), OnAbort(UE_MT_SCOPED_READ_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::OnAbort([&]
			{
				UE_MT_SCOPED_READ_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Scope(UE_MT_SCOPED_WRITE_ACCESS), OnAbort(UE_MT_SCOPED_WRITE_ACCESS), Abort)")
	{
		TestType Detector;
		AutoRTFM::Testing::Abort([&]
		{
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			}

			AutoRTFM::OnAbort([&]
			{
				UE_MT_SCOPED_WRITE_ACCESS(Detector);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Create, UE_MT_SCOPED_READ_ACCESS)")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TestType Detector;
			UE_MT_SCOPED_READ_ACCESS(Detector);
		});
	}

	SECTION("Transact(Create, UE_MT_SCOPED_READ_ACCESS, Abort)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TestType Detector;
			UE_MT_SCOPED_READ_ACCESS(Detector);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Transact(Create, UE_MT_SCOPED_WRITE_ACCESS)")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TestType Detector;
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
		});
	}

	SECTION("Transact(Create, UE_MT_SCOPED_WRITE_ACCESS, Abort)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TestType Detector;
			UE_MT_SCOPED_WRITE_ACCESS(Detector);
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("UE_MT_SCOPED_READ_ACCESS, destruct, memzero, reconstruct")
	{
		TestType Detector;
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				{
					// Read-lock and then unlock
					UE_MT_SCOPED_READ_ACCESS(Detector);
				}
				Detector.~TestType();
				memset(&Detector, 0, sizeof(Detector));
				new (&Detector) TestType();
			});
		}
		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				{
					// Read-lock and then unlock
					UE_MT_SCOPED_READ_ACCESS(Detector);
				}
				Detector.~TestType();
				memset(&Detector, 0, sizeof(Detector));
				new (&Detector) TestType();
				AutoRTFM::AbortTransaction();
			});
		}
	}

	SECTION("UE_MT_SCOPED_WRITE_ACCESS, destruct, memzero, reconstruct")
	{
		TestType Detector;
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				{
					// Write-lock and then unlock
					UE_MT_SCOPED_WRITE_ACCESS(Detector);
				}
				Detector.~TestType();
				memset(&Detector, 0, sizeof(Detector));
				new (&Detector) TestType();
			});
		}
		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				{
					// Write-lock and then unlock
					UE_MT_SCOPED_WRITE_ACCESS(Detector);
				}
				Detector.~TestType();
				memset(&Detector, 0, sizeof(Detector));
				new (&Detector) TestType();
				AutoRTFM::AbortTransaction();
			});
		}
	}
}

#endif // ENABLE_MT_DETECTOR
