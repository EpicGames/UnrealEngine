// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

#include <math.h>
#include <utility>

namespace
{

// Check calls Function outside a transaction, in an aborted transaction and
// committed transaction. The value returned when called in the committed
// transaction is expected to match the value returned when called outside the
// transaction.
// Function must be the deterministic function with the signature 'T()'.
template<typename FUNC>
void Check(AUTORTFM_IMPLICIT_ENABLE FUNC&& Function)
{
	using T = decltype(Function());
	const T Expected = Function();
	const T Zero{};

	SECTION("With Abort")
	{
		T Got = Zero;
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Got = Function();
				AutoRTFM::AbortTransaction();
			});

		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		REQUIRE(Zero == Got);
	}

	SECTION("With Commit")
	{
		T Got = Zero;

		AutoRTFM::Testing::Commit([&] { Got = Function(); });

		REQUIRE(Expected == Got);
	}
}

}  // anonymous namespace

TEST_CASE("Math.sqrt")
{
	SECTION("float")  { Check([]{ return sqrt(0.42f); }); };
	SECTION("double") { Check([]{ return sqrt(0.42); }); };
}

TEST_CASE("Math.sqrtf")
{
	Check([]{ return sqrtf(0.42f); });
}

TEST_CASE("Math.sin")
{
	SECTION("float")  { Check([]{ return sin(0.42f); }); }
	SECTION("double") { Check([]{ return sin(0.42); }); }
}

TEST_CASE("Math.sinf")
{
	Check([]{ return sinf(0.42f); });
}

TEST_CASE("Math.cos")
{
	SECTION("float")  { Check([]{ return cos(0.42f); }); }
	SECTION("double") { Check([]{ return cos(0.42); }); }
}

TEST_CASE("Math.cosf")
{
	Check([]{ return cosf(0.42f); });
}

TEST_CASE("Math.tan")
{
	SECTION("float")  { Check([]{ return tan(0.42f); }); }
	SECTION("double") { Check([]{ return tan(0.42); }); }
}

TEST_CASE("Math.tanf")
{
	Check([]{ return tanf(0.42f); });
}

TEST_CASE("Math.asin")
{
	SECTION("float")  { Check([]{ return asin(0.42f); }); }
	SECTION("double") { Check([]{ return asin(0.42); }); }
}

TEST_CASE("Math.asinf")
{
	Check([]{ return asinf(0.42f); });
}

TEST_CASE("Math.acos")
{
	SECTION("float")  { Check([]{ return acos(0.42f); }); }
	SECTION("double") { Check([]{ return acos(0.42); }); }
}

TEST_CASE("Math.acosf")
{
	Check([]{ return acosf(0.42f); });
}

TEST_CASE("Math.atan")
{
	SECTION("float")  { Check([]{ return atan(0.42f); }); }
	SECTION("double") { Check([]{ return atan(0.42); }); }
}

TEST_CASE("Math.atanf")
{
	Check([]{ return atanf(0.42f); });
}

TEST_CASE("Math.atan2")
{
	SECTION("float")  { Check([]{ return atan2(0.42f, 0.42f); }); }
	SECTION("double") { Check([]{ return atan2(0.42, 0.42); }); }
}

TEST_CASE("Math.atan2f")
{
	Check([]{ return atan2f(0.42f, 0.24f); });
}

TEST_CASE("Math.sinh")
{
	SECTION("float")  { Check([]{ return sinh(0.42f); }); }
	SECTION("double") { Check([]{ return sinh(0.42); }); }
}

TEST_CASE("Math.sinhf")
{
	Check([]{ return sinhf(0.42f); });
}

TEST_CASE("Math.cosh")
{
	SECTION("float")  { Check([]{ return cosh(0.42f); }); }
	SECTION("double") { Check([]{ return cosh(0.42); }); }
}

TEST_CASE("Math.coshf")
{
	Check([]{ return coshf(0.42f); });
}

TEST_CASE("Math.tanh")
{
	SECTION("float")  { Check([]{ return tanh(0.42f); }); }
	SECTION("double") { Check([]{ return tanh(0.42); }); }
}

TEST_CASE("Math.tanhf")
{
	Check([]{ return tanhf(0.42f); });
}

TEST_CASE("Math.asinh")
{
	SECTION("float")  { Check([]{ return asinh(0.42f); }); }
	SECTION("double") { Check([]{ return asinh(0.42); }); }
}

TEST_CASE("Math.asinhf")
{
	Check([]{ return asinhf(0.42f); });
}

TEST_CASE("Math.acosh")
{
	SECTION("float")  { Check([]{ return acosh(4.2f); }); }
	SECTION("double") { Check([]{ return acosh(4.2); }); }
}

TEST_CASE("Math.acoshf")
{
	Check([]{ return acoshf(4.2f); });
}

TEST_CASE("Math.atanh")
{
	SECTION("float")  { Check([]{ return atanh(0.42f); }); }
	SECTION("double") { Check([]{ return atanh(0.42); }); }
}

TEST_CASE("Math.atanhf")
{
	Check([]{ return atanhf(0.42f); });
}

TEST_CASE("Math.exp")
{
	SECTION("float")  { Check([]{ return exp(0.42f); }); }
	SECTION("double") { Check([]{ return exp(0.42); }); }
}

TEST_CASE("Math.expf")
{
	Check([]{ return expf(0.42f); });
}

TEST_CASE("Math.log")
{
	SECTION("float")  { Check([]{ return log(0.42f); }); }
	SECTION("double") { Check([]{ return log(0.42); }); }
}

TEST_CASE("Math.pow")
{
	SECTION("float")  { Check([]{ return pow(0.42f, 0.42f); }); }
	SECTION("double") { Check([]{ return pow(0.42, 0.42); }); }
}

TEST_CASE("Math.powf")
{
	Check([]{ return powf(0.42f, 0.24f); });
}

TEST_CASE("Math.logf")
{
	Check([]{ return logf(0.42f); });
}

TEST_CASE("Math.llrint")
{
	SECTION("float")  { Check([]{ return llrint(0.42f); }); }
	SECTION("double") { Check([]{ return llrint(0.42); }); }
}

TEST_CASE("Math.llrintf")
{
	Check([]{ return llrintf(0.42f); });
}

TEST_CASE("Math.fmod")
{
	SECTION("float")  { Check([]{ return fmod(0.42f, 0.42f); }); }
	SECTION("double") { Check([]{ return fmod(0.42, 0.42); }); }
}

TEST_CASE("Math.fmodf")
{
	Check([]{ return fmodf(0.42f, 0.24f); });
}

TEST_CASE("Math.fmodl")
{
	Check([]{ return fmodl(0.42, 0.24); });
}

TEST_CASE("Math.modf")
{
	SECTION("float")
	{ 
		Check([]
		{
			float A = 0;
			float B = modf(4.2f, &A);
			return std::make_pair(A, B); 
		});
	}
	SECTION("long double")
	{
		Check([]
		{
			long double A = 0;
			long double B = modf(4.2, &A);
			return std::make_pair(A, B); 
		});
	}
	SECTION("double")
	{
		Check([]
		{
			double A = 0;
			double B = modf(4.2, &A);
			return std::make_pair(A, B); 
		});
	}
}

TEST_CASE("Math.modff")
{
	Check([]
	{
		float A = 0;
		float B = modff(4.2f, &A);
		return std::make_pair(A, B); 
	});
}

TEST_CASE("Math.rand")
{
	// it's near impossible to test the value returned by rand(), so just check
	// that calling it a transaction doesn't explode.

	SECTION("With Abort")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				rand();
				AutoRTFM::AbortTransaction();
			});
		REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	}

	SECTION("With Commit")
	{
		AutoRTFM::Testing::Commit([&] { rand(); });
	}
}
