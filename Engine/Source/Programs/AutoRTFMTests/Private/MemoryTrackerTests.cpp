// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "IntervalTestUtils.h"
#include "MemoryTracker.h"

#include <random>

using FMemoryTracker = AutoRTFM::FMemoryTracker;

TEST_CASE("MemoryTracker")
{
	AutoRTFM::UnreachableIfClosed();

	// Convenience lambda to convert a uintptr_t to a void*
	auto Ptr = [](uintptr_t Address) { return reinterpret_cast<void*>(Address); };

	auto Insert = [Ptr](FMemoryTracker& Tree, uintptr_t Address, uintptr_t Size)
	{
		Tree.Insert(Ptr(Address), Size);
	};

	// Checks that the tracker has all of the expected intervals, and that Contains() returns
	// the correct value for various addresses.
	auto Validate = [&](FMemoryTracker& Tracker, std::vector<FInterval> Expected)
	{
		std::vector<FInterval> Got;
		for (FInterval Entry : Tracker)
		{
			Got.push_back(Entry);
		}

		REQUIRE(Expected.empty() == Tracker.IsEmpty());
		for (size_t I = 0, N = std::min(Expected.size(), Got.size()); I < N; I++)
		{
			REQUIRE(Expected[I].Start == Got[I].Start);
			REQUIRE(Expected[I].End == Got[I].End);
		}
		REQUIRE(Expected.size() == Got.size());
		for (size_t Index = 0; Index < Expected.size(); Index++)
		{
			const FInterval Entry = Expected[Index];
			const bool bTouchingPrev = Index > 0 && Expected[Index - 1].End == Entry.Start;
			const bool bTouchingNext = Index < (Expected.size() - 1) && Expected[Index + 1].Start == Entry.End;
			REQUIRE(true == Tracker.Contains(Ptr(Entry.Start), 1));
			REQUIRE(true == Tracker.Contains(Ptr(Entry.Start), Entry.Size()));
			REQUIRE(bTouchingPrev == Tracker.Contains(Ptr(Entry.Start - 1), 1));
			REQUIRE(bTouchingNext == Tracker.Contains(Ptr(Entry.End), 1));
		}
	};

	FMemoryTracker Tracker;

	Validate(Tracker, {});

	SECTION("Insert")
	{
		Tracker.Reset();
		Validate(Tracker, {});

		// New intervals
		Insert(Tracker, 40, 10);
		Validate(Tracker, {{40, 50}});

		Insert(Tracker, 20, 10);
		Validate(Tracker, {{20, 30}, {40, 50}});

		Insert(Tracker, 60, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}});

		Insert(Tracker, 80, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		// Re-inserted intervals
		Insert(Tracker, 20, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		Insert(Tracker, 40, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		Insert(Tracker, 60, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		Insert(Tracker, 80, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		// Expanding intervals without overlap
		Insert(Tracker, 15, 5);
		Validate(Tracker, {{15, 30}, {40, 50}, {60, 70}, {80, 90}});

		Insert(Tracker, 30, 5);
		Validate(Tracker, {{15, 35}, {40, 50}, {60, 70}, {80, 90}});

		Insert(Tracker, 55, 5);
		Validate(Tracker, {{15, 35}, {40, 50}, {55, 70}, {80, 90}});

		Insert(Tracker, 70, 5);
		Validate(Tracker, {{15, 35}, {40, 50}, {55, 75}, {80, 90}});

		// Expanding intervals with overlap
		Insert(Tracker, 35, 5);
		Validate(Tracker, {{15, 35}, {35, 50}, {55, 75}, {80, 90}});

		Insert(Tracker, 50, 5);
		Validate(Tracker, {{15, 35}, {35, 55}, {55, 75}, {80, 90}});

		Insert(Tracker, 75, 5);
		Validate(Tracker, {{15, 35}, {35, 55}, {55, 80}, {80, 90}});
	}

	SECTION("Merge")
	{
		Insert(Tracker, 20, 10);
		Insert(Tracker, 40, 10);
		Insert(Tracker, 60, 10);
		Insert(Tracker, 80, 10);
		Validate(Tracker, {{20, 30}, {40, 50}, {60, 70}, {80, 90}});

		SECTION("Intertwined")
		{
			FMemoryTracker TrackerB;
			Insert(TrackerB, 10, 5);
			Insert(TrackerB, 33, 6);
			Insert(TrackerB, 54, 2);
			Insert(TrackerB, 73, 3);
			Insert(TrackerB, 95, 2);
			Validate(TrackerB, {{10, 15}, {33, 39}, {54, 56}, {73, 76}, {95, 97}});

			Tracker.Merge(TrackerB);
			Validate(Tracker, {{10, 15}, {20, 30}, {33, 39}, {40, 50}, {54, 56}, {60, 70}, {73, 76}, {80, 90}, {95, 97}});
		}

		SECTION("Expand")
		{
			FMemoryTracker TrackerB;
			Insert(TrackerB, 15, 5);
			Insert(TrackerB, 30, 3);
			Insert(TrackerB, 37, 3);
			Insert(TrackerB, 50, 2);
			Insert(TrackerB, 55, 5);
			Insert(TrackerB, 70, 4);
			Insert(TrackerB, 78, 2);
			Insert(TrackerB, 90, 3);
			Validate(TrackerB, {{15, 20}, {30, 33}, {37, 40}, {50, 52}, {55, 60}, {70, 74}, {78, 80}, {90, 93}});

			Tracker.Merge(TrackerB);
			Validate(Tracker, {{15, 33}, {37, 52}, {55, 74}, {78, 93}});
		}

		SECTION("Overlap")
		{
			FMemoryTracker TrackerB;
			Insert(TrackerB, 30, 10);
			Insert(TrackerB, 50, 10);
			Insert(TrackerB, 70, 10);
			Validate(TrackerB, {{30, 40}, {50, 60}, {70, 80}});

			Tracker.Merge(TrackerB);
			Validate(Tracker, {{20, 30}, {30, 60}, {60, 80}, {80, 90}});
		}
	}

	SECTION("Soak")
	{
		for (int Pass = 0; Pass < 32; Pass++)
		{
			Tracker.Reset();
			Validate(Tracker, {});

			FTreeIntervals Intervals = BuildIntervals(/* Seed */ Pass, /* MaxUsed */ 1000);

			for (FInterval Interval : Intervals.Used)
			{
				Insert(Tracker, Interval.Start, Interval.Size());
			}

			Intervals.SortAndValidate();

			for (const FInterval& FreeInterval : Intervals.Free)
			{
				REQUIRE(!Tracker.Contains(Ptr(FreeInterval.Start), FreeInterval.Size()));
				REQUIRE(!Tracker.Contains(Ptr(FreeInterval.Start), 1));
				REQUIRE(!Tracker.Contains(Ptr(FreeInterval.End - 1), 1));
			}
			for (const FInterval& UsedInterval : Intervals.Used)
			{
				REQUIRE(Tracker.Contains(Ptr(UsedInterval.Start), UsedInterval.Size()));
				REQUIRE(Tracker.Contains(Ptr(UsedInterval.Start), 1));
				REQUIRE(Tracker.Contains(Ptr(UsedInterval.End - 1), 1));
			}
		}
	}

	Tracker.Reset();
	Validate(Tracker, {});
}

TEST_CASE("MemoryTracker.Benchmarks")
{
	// Convenience lambda to convert a uintptr_t to a void*
	auto Ptr = [](uintptr_t Address) { return reinterpret_cast<void*>(Address); };

	FMemoryTracker Tracker;

	// Populates Tracker with a random set of intervals.
	// Returns the FTreeIntervals used to populate the tracker.
	auto PopulateTracker = [&]() -> FTreeIntervals
	{
		const FTreeIntervals Intervals = BuildIntervals(/* Seed */ 1234, /* MaxUsed */ 10'000);
		for (FInterval Interval : Intervals.Used)
		{
			Tracker.Insert(Ptr(Interval.Start), Interval.Size());
		}
		return Intervals;
	};

	BENCHMARK_ADVANCED("Insert")(Catch::Benchmark::Chronometer Meter)
	{
		const std::vector<FInterval> Intervals = PopulateTracker().Used;
		Meter.measure([&]
		{
			for (FInterval Interval : Intervals)
			{
				Tracker.Insert(Ptr(Interval.Start), Interval.Size());
			}
		});
	};

	// Populates Tracker with a random set of intervals.
	// If bUsed is true, returns a random set of addresses in those intervals,
	// otherwise a random set of addresses outside of those intervals.
	auto PopulateTrackerAndPickAddresses = [&](bool bUsed) -> std::vector<void*>
	{
		std::mt19937 Rand(/* Seed */ 4242);

		const std::vector<FInterval> Intervals = bUsed ? PopulateTracker().Used : PopulateTracker().Free;

		constexpr size_t NumAddresses = 10'000;
		std::vector<void*> Addresses(NumAddresses);
		for (size_t I = 0; I < NumAddresses; I++)
		{
			FInterval Interval = Intervals[Rand() % Intervals.size()];
			Addresses[I] = Ptr(Interval.Start + (Rand() % Interval.Size()));
		}
		return Addresses;
	};

	BENCHMARK_ADVANCED("Contains.Used")(Catch::Benchmark::Chronometer Meter)
	{
		std::vector<void*> Addresses = PopulateTrackerAndPickAddresses(/* bUsed */ true);
		Meter.measure([&]
		{
			for (void* Address : Addresses)
			{
				Tracker.Contains(Address, 4);
			}
		});
	};

	BENCHMARK_ADVANCED("Contains.Free")(Catch::Benchmark::Chronometer Meter)
	{
		std::vector<void*> Addresses = PopulateTrackerAndPickAddresses(/* bUsed */ false);
		Meter.measure([&]
		{
			for (void* Address : Addresses)
			{
				Tracker.Contains(Address, 4);
			}
		});
	};

	BENCHMARK_ADVANCED("Merge")(Catch::Benchmark::Chronometer Meter)
	{
		PopulateTracker();
		FMemoryTracker OtherTracker;
		{
			const FTreeIntervals Intervals = BuildIntervals(/* Seed */ 1234, /* MaxUsed */ 10'000);
			for (FInterval Interval : Intervals.Used)
			{
				OtherTracker.Insert(Ptr(Interval.Start), Interval.Size());
			}
		}

		Meter.measure([&]
		{
			Tracker.Merge(OtherTracker);
		});
	};
}
