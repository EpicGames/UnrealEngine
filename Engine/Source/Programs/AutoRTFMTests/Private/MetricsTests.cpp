// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFMTestUtils.h"
#include "AutoRTFM/Metrics.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "MetricsPriv.h"
#include "Utils.h"

#include <algorithm>
#include <stdio.h>
#include <string.h>

TEST_CASE("Metrics.TransactionCounts")
{
	// Automatic retries inflate our metrics, because everything is run twice. We can get accurate histograms by disabling retry.
	AUTORTFM_SCOPED_DISABLE_RETRY();

	AutoRTFM::ResetAutoRTFMMetrics();

	SECTION("Top-level")
	{
		AutoRTFM::Transact([] {});
		AutoRTFM::Transact([] { AutoRTFM::AbortTransaction(); });

		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 2);
		REQUIRE(Metrics.NumTransactionsStarted == 2);
		REQUIRE(Metrics.NumTransactionsCommitted == 1);
		REQUIRE(Metrics.NumTransactionsAborted == 1);
		REQUIRE(Metrics.NumRequestedAborts == 1);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Nesting")
	{
		AutoRTFM::Transact([]
		{
			AutoRTFM::Transact([] {});
			AutoRTFM::Transact([] {});
			AutoRTFM::Transact([]
			{
				AutoRTFM::Transact([] {});
			});
		});

		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 5);
		REQUIRE(Metrics.NumTransactionsCommitted == 5);
		REQUIRE(Metrics.NumTransactionsAborted == 0);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Scoped rollback")
	{
		AutoRTFM::TransactThenOpen([]
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::AbortTransaction();
		});
		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 2);
		REQUIRE(Metrics.NumTransactionsCommitted == 1);
		REQUIRE(Metrics.NumTransactionsAborted == 1);
		REQUIRE(Metrics.NumRequestedAborts == 1);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Scoped commit")
	{
		AutoRTFM::TransactThenOpen([]
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::CommitTransaction();
		});
		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 2);
		REQUIRE(Metrics.NumTransactionsCommitted == 2);
		REQUIRE(Metrics.NumTransactionsAborted == 0);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Cascading abort")
	{
		AutoRTFM::Transact([]
		{
			AutoRTFM::Transact([]
			{
				AutoRTFM::CascadingAbortTransaction();
			});
		});
		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 2);
		REQUIRE(Metrics.NumTransactionsCommitted == 0);
		REQUIRE(Metrics.NumTransactionsAborted == 2);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 1);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Cascading retry")
	{
		bool bShouldRetry = true;

		AutoRTFM::Transact([&]
		{
			AutoRTFM::OnRetry([&]
			{
				bShouldRetry = false;
			});

			AutoRTFM::Transact([&]
			{
				if (bShouldRetry)
				{
					AutoRTFM::CascadingRetryTransaction();
				}
			});
		});
		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 2);
		REQUIRE(Metrics.NumTransactionsStarted == 4);
		REQUIRE(Metrics.NumTransactionsCommitted == 2);
		REQUIRE(Metrics.NumTransactionsAborted == 2);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 1);
		REQUIRE(Metrics.NumTransactionalViolations == 0);
	}

	SECTION("Transactional violation - no closed function")
	{
		AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
		AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2(false);

		AutoRTFM::Transact([]
		{
			(void)fopen("fopen() is not supported in a closed transaction", "rb");
		});

		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 1);
		REQUIRE(Metrics.NumTransactionsCommitted == 0);
		REQUIRE(Metrics.NumTransactionsAborted == 1);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 1);
	}

	SECTION("Transactional violation - unreachable")
	{
		AutoRTFMTestUtils::FScopedInternalAbortAction Scoped1(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
		AutoRTFMTestUtils::FScopedEnsureOnInternalAbort Scoped2(false);

		AutoRTFM::Transact([]
		{
			AutoRTFM::UnreachableIfClosed();
		});

		AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(Metrics.NumTopLevelTransactionsCompleted == 1);
		REQUIRE(Metrics.NumTransactionsStarted == 1);
		REQUIRE(Metrics.NumTransactionsCommitted == 0);
		REQUIRE(Metrics.NumTransactionsAborted == 1);
		REQUIRE(Metrics.NumRequestedAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingAborts == 0);
		REQUIRE(Metrics.NumRequestedCascadingRetries == 0);
		REQUIRE(Metrics.NumTransactionalViolations == 1);
	}
}

TEST_CASE("Metrics.Histogram")
{
	TArray<uint8> Scratchpad;
	Scratchpad.SetNum(10'000'000);

	// Verify that our histogram bucket array is large enough for any amount of memory we could potentially allocate.
	using HistogramBucketArray = AutoRTFM::FAutoRTFMMetrics::HistogramBucketArray;
	static_assert(std::size(HistogramBucketArray{}) > AutoRTFM::Log2(UINTPTR_MAX));

	// Automatic retries inflate our metrics, because everything is run twice. We can get accurate histograms by disabling retry.
	AUTORTFM_SCOPED_DISABLE_RETRY();

	// Start with a clean slate by resetting the metrics.
	AutoRTFM::ResetAutoRTFMMetrics();
	AutoRTFM::FAutoRTFMMetrics ZeroMetrics = AutoRTFM::GetAutoRTFMMetrics();

	// Make sure that after a reset, the peak memory usage is zero and the histogram has no entries.
	REQUIRE(ZeroMetrics.OverallPeakMemoryUsage == 0);
	HistogramBucketArray ZeroHistogram = {};
	static_assert(sizeof(ZeroMetrics.MemoryUsageHistogram) == sizeof(ZeroHistogram));
	REQUIRE(std::equal(std::begin(ZeroMetrics.MemoryUsageHistogram), std::end(ZeroMetrics.MemoryUsageHistogram), std::begin(ZeroHistogram), std::end(ZeroHistogram)));

	// Run a no-op transaction to get a baseline of "peak" memory usage when nothing is going on.
	AutoRTFM::Testing::Commit([] {});
	AutoRTFM::FAutoRTFMMetrics BaselineMetrics = AutoRTFM::GetAutoRTFMMetrics();

	// Make sure that the histogram has exactly one entry for the current peak.
	HistogramBucketArray BaselineHistogram = {};
	BaselineHistogram[AutoRTFM::Log2(BaselineMetrics.OverallPeakMemoryUsage)] = 1;
	REQUIRE(std::equal(std::begin(BaselineMetrics.MemoryUsageHistogram), std::end(BaselineMetrics.MemoryUsageHistogram), std::begin(BaselineHistogram), std::end(BaselineHistogram)));

	SECTION("Abort")
	{
		// Run a transaction which aborts after modifying a large amount of memory. This should affect our peak
		// by at _least_ that much memory, since the write log needs to store every byte touched.
		AutoRTFM::Testing::Abort([&]
		{
			memset(Scratchpad.GetData(), 0x55, Scratchpad.NumBytes());
			AutoRTFM::AbortTransaction();
		});

		// Make sure that AutoRTFM peak memory increased by the size of the scratchpad.
		AutoRTFM::FAutoRTFMMetrics PostAbortMetrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(PostAbortMetrics.OverallPeakMemoryUsage > BaselineMetrics.OverallPeakMemoryUsage + Scratchpad.NumBytes());

		// Make sure that the histogram has two entries--one for the current transaction, and one for the baseline.
		HistogramBucketArray PostAbortHistogram = {};
		PostAbortHistogram[AutoRTFM::Log2(BaselineMetrics.OverallPeakMemoryUsage)] += 1;
		PostAbortHistogram[AutoRTFM::Log2(PostAbortMetrics.OverallPeakMemoryUsage)] += 1;
		REQUIRE(std::equal(std::begin(PostAbortMetrics.MemoryUsageHistogram), std::end(PostAbortMetrics.MemoryUsageHistogram), std::begin(PostAbortHistogram), std::end(PostAbortHistogram)));
	}

	SECTION("Commit")
	{
		// Run two transactions which commit after modifying a large amount of memory.
		for (int Count=0; Count<2; ++Count)
		{
			AutoRTFM::Testing::Commit([&]
			{
				memset(Scratchpad.GetData(), 0xCC, Scratchpad.NumBytes());
			});
		}

		// Make sure that AutoRTFM peak memory increased by the size of the scratchpad.
		AutoRTFM::FAutoRTFMMetrics PostCommitMetrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(PostCommitMetrics.OverallPeakMemoryUsage > BaselineMetrics.OverallPeakMemoryUsage + Scratchpad.NumBytes());

		// Make sure that the histogram has three entries--two for these transactions, and one for the baseline.
		HistogramBucketArray PostCommitHistogram = {};
		PostCommitHistogram[AutoRTFM::Log2(BaselineMetrics.OverallPeakMemoryUsage)] += 1;
		PostCommitHistogram[AutoRTFM::Log2(PostCommitMetrics.OverallPeakMemoryUsage)] += 2;
		REQUIRE(std::equal(std::begin(PostCommitMetrics.MemoryUsageHistogram), std::end(PostCommitMetrics.MemoryUsageHistogram), std::begin(PostCommitHistogram), std::end(PostCommitHistogram)));
	}

	SECTION("Abort, transacting from the open")
	{
		// Run three transactions which abort after recording an open write on the scratchpad. (Actually modifying the scratchpad is superfluous.)
		for (int Count=0; Count<3; ++Count)
		{
			AutoRTFM::TransactThenOpen([&]
			{
				AutoRTFM::StartTransaction();
				AutoRTFM::RecordOpenWrite(Scratchpad.GetData(), Scratchpad.NumBytes());
				AutoRTFM::AbortTransaction();
			});
		}

		// Make sure that AutoRTFM peak memory increased by the size of the scratchpad.
		AutoRTFM::FAutoRTFMMetrics PostAbortMetrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(PostAbortMetrics.OverallPeakMemoryUsage > BaselineMetrics.OverallPeakMemoryUsage + Scratchpad.NumBytes());

		// Make sure that the histogram has four entries--three for these transactions, and one for the baseline.
		HistogramBucketArray PostAbortHistogram = {};
		PostAbortHistogram[AutoRTFM::Log2(BaselineMetrics.OverallPeakMemoryUsage)] += 1;
		PostAbortHistogram[AutoRTFM::Log2(PostAbortMetrics.OverallPeakMemoryUsage)] += 3;
		REQUIRE(std::equal(std::begin(PostAbortMetrics.MemoryUsageHistogram), std::end(PostAbortMetrics.MemoryUsageHistogram), std::begin(PostAbortHistogram), std::end(PostAbortHistogram)));
	}

	SECTION("Commit, transacting from the open")
	{
		// Run four transactions which commit after recording an open write on the scratchpad. (Actually modifying the scratchpad is superfluous.)
		for (int Count=0; Count<4; ++Count)
		{
			AutoRTFM::TransactThenOpen([&]
			{
				AutoRTFM::StartTransaction();
				AutoRTFM::RecordOpenWrite(Scratchpad.GetData(), Scratchpad.NumBytes());
				AutoRTFM::CommitTransaction();
			});
		}

		// Make sure that AutoRTFM peak memory increased by the size of the scratchpad.
		AutoRTFM::FAutoRTFMMetrics PostCommitMetrics = AutoRTFM::GetAutoRTFMMetrics();
		REQUIRE(PostCommitMetrics.OverallPeakMemoryUsage > BaselineMetrics.OverallPeakMemoryUsage + Scratchpad.NumBytes());

		// Make sure that the histogram has five entries--four for these transactions, and one for the baseline.
		HistogramBucketArray PostCommitHistogram = {};
		PostCommitHistogram[AutoRTFM::Log2(BaselineMetrics.OverallPeakMemoryUsage)] += 1;
		PostCommitHistogram[AutoRTFM::Log2(PostCommitMetrics.OverallPeakMemoryUsage)] += 4;
		REQUIRE(std::equal(std::begin(PostCommitMetrics.MemoryUsageHistogram), std::end(PostCommitMetrics.MemoryUsageHistogram), std::begin(PostCommitHistogram), std::end(PostCommitHistogram)));
	}
}

TEST_CASE("Metrics.ConvertToCSV")
{
	AutoRTFM::FAutoRTFMMetrics Metrics;

	Metrics.NumTopLevelTransactionsCompleted  = 1001;
	Metrics.NumTransactionsStarted            = 1002;
	Metrics.NumTransactionsCommitted          = 1003;
	Metrics.NumTransactionsAborted            = 1004;
	Metrics.NumRequestedAborts                = 1005;
	Metrics.NumRequestedCascadingAborts       = 1006;
	Metrics.NumRequestedCascadingRetries      = 1007;
	Metrics.NumTransactionalViolations        = 1008;
	Metrics.OverallPeakMemoryUsage            = 1009;
	Metrics.ActiveTransactionPeakMemoryUsage  = 1010;

	uint32 Filler = 2001;
	for (uint32_t& Entry : Metrics.MemoryUsageHistogram)
	{
		Entry = Filler++;
	}

	char Text[2048];
	REQUIRE(AutoRTFM::ConvertMetricsToCSV(Metrics, Text, std::size(Text)));
	REQUIRE(0 == strcmp(Text,
		"1001,1002,1003,1004,1005,1006,1007,1008,1009,1010,"
		"2001,2002,2003,2004,2005,2006,2007,2008,2009,2010,"
		"2011,2012,2013,2014,2015,2016,2017,2018,2019,2020,"
		"2021,2022,2023,2024,2025,2026,2027,2028,2029,2030,"
		"2031,2032,2033,2034,2035,2036,2037,2038,2039,2040,"
		"2041,2042,2043,2044,2045,2046,2047,2048,2049,2050,"
		"2051,2052,2053,2054,2055,2056,2057,2058,2059,2060,"
		"2061,2062,2063,2064"));
}

TEST_CASE("Metrics.AutoRetryCountsTopLevelCorrectly")
{
	AutoRTFM::ResetAutoRTFMMetrics();
	AutoRTFM::Transact([&] {});
	AutoRTFM::FAutoRTFMMetrics Metrics = AutoRTFM::GetAutoRTFMMetrics();

	// If retries are enabled, this will evaluate as `2 == 1+1` due to the automatic abort-and-retry mechanism inside Transact.
	REQUIRE(Metrics.NumTopLevelTransactionsCompleted == Metrics.NumTransactionsCommitted + Metrics.NumTransactionsAborted);

	// Confirm that the number of histogram entries matches the number of top-level transactions.
	size_t HistogramEntries = 0;
	for (uint32_t Entry : Metrics.MemoryUsageHistogram)
	{
		HistogramEntries += Entry;
	}

	REQUIRE(Metrics.NumTopLevelTransactionsCompleted == HistogramEntries);
}