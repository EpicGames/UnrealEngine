// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/SharedLock.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "Async/TransactionallySafeMutex.h"
#include "Async/TransactionallySafeSharedMutex.h"
#include "Async/TransactionallySafeSharedRecursiveMutex.h"
#include "Async/TransactionallySafeMutexPool.h"

#include <atomic>
#include <condition_variable>
#include <cstring>
#include <memory>
#include <mutex>
#include <thread>
#include <type_traits>

TEMPLATE_TEST_CASE("MutexTests", "", 
	UE::FTransactionallySafeMutex,
	UE::FTransactionallySafeRecursiveMutex,
	UE::FTransactionallySafeSharedMutex,
	UE::FTransactionallySafeSharedRecursiveMutex)
{
	using MutexType = TestType;
	
	// TODO(SOL-7661): Unify the transactionally-safe mutex implementations so
	// we can have a consistent set of behaviors across all variants.

	// Some mutex implementations do not currently support having an unbalanced
	// number of calls to Lock() / Unlock().
	constexpr bool bSupportsUnbalancedLockingInTransaction = std::is_same_v<MutexType, UE::FTransactionallySafeMutex>;

	// Some mutex implementations do not currently support locking and unlocking
	// across open / closed boundaries.
	constexpr bool bSupportsMixedOpenCloseLocking = std::is_same_v<MutexType, UE::FTransactionallySafeMutex>;

	// True if the mutex supports TryLock() within a transaction
	constexpr bool bSupportsTryLockInTransaction = true;

	// Some mutex implementations do not currently support the IsLocked() method
	constexpr bool bSupportsIsLocked = UE::Private::CAllowsIsLocked<MutexType>;

	// True if the mutex supports recursive locking
	constexpr bool bIsRecursive =
		std::is_same_v<MutexType, UE::FTransactionallySafeRecursiveMutex> ||
		std::is_same_v<MutexType, UE::FTransactionallySafeSharedRecursiveMutex>;

	// True if the mutex is shared
	constexpr bool bIsShared =
		std::is_same_v<MutexType, UE::FTransactionallySafeSharedMutex> ||
		std::is_same_v<MutexType, UE::FTransactionallySafeSharedRecursiveMutex>;

	auto RequireLocked = [](MutexType& Mutex)
	{
		if constexpr (bSupportsIsLocked)
		{
			REQUIRE(Mutex.IsLocked());
		}
		else
		{
			// No way to probe the mutex locked state.
			// As a best effort attempt to unlock and re-lock.
			Mutex.Unlock();
			Mutex.Lock();
		}
	};

	auto RequireNotLocked = [](MutexType& Mutex)
	{
		if constexpr (bSupportsIsLocked)
		{
			REQUIRE(!Mutex.IsLocked());
		}
		else
		{
			// No way to probe the mutex locked state.
			// As a best effort attempt to lock and unlock.
			Mutex.Lock();
			Mutex.Unlock();
		}
	};

	auto RequireLockSharedState = [](MutexType& Mutex, bool bState)
	{
		if constexpr (bIsShared)
		{
			REQUIRE(Mutex.IsLockShared() == bState);
		}
		else
		{
			FAIL("unsupported lock type");
		}
	};
	auto RequireLockShared = [&RequireLockSharedState](MutexType& Mutex)
	{
		RequireLockSharedState(Mutex, true);
	};
	auto RequireNotSharedLocked = [&RequireLockSharedState](MutexType& Mutex)
	{
		RequireLockSharedState(Mutex, false);
	};

	auto TryLockShared = [](MutexType& Mutex, auto& Link) -> bool
	{
		if constexpr (bIsShared && bIsRecursive)
		{
			return Mutex.TryLockShared(Link);
		}
		else if constexpr (bIsShared && !bIsRecursive)
		{
			return Mutex.TryLockShared();
		}
		else
		{
			FAIL("unsupported lock type");
			return false;
		}
	};

	auto UnlockShared = [](MutexType& Mutex, auto& Link)
	{
		if constexpr (bIsShared && bIsRecursive)
		{
			Mutex.UnlockShared(Link);
		}
		else if constexpr (bIsShared && !bIsRecursive)
		{
			Mutex.UnlockShared();
		}
		else
		{
			FAIL("unsupported lock type");
		}
	};
	
	int NumLocksUsed = UE::Private::TransactionallySafeMutexPool::GetNumUsed();

	SECTION("Construct Outside Transaction")
	{
		MutexType Mutex;

		AutoRTFM::Testing::Abort([&]
		{
			UE::TScopeLock Lock(Mutex);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			UE::TScopeLock Lock(Mutex);
		});
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Construct Inside Transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			MutexType Mutex;
			UE::TScopeLock Lock(Mutex);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			MutexType Mutex;
			UE::TScopeLock Lock(Mutex);
		});
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("IsLocked Outside Transaction")
	{
		MutexType Mutex;

		RequireNotLocked(Mutex);
		Mutex.Lock();
		RequireLocked(Mutex);
		Mutex.Unlock();
		RequireNotLocked(Mutex);
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("IsLocked Inside Transaction")
	{
		MutexType Mutex;
		RequireNotLocked(Mutex);

		AutoRTFM::Testing::Commit([&]
		{
			RequireNotLocked(Mutex);
			Mutex.Lock();
			RequireLocked(Mutex);
			Mutex.Unlock();
			RequireLocked(Mutex);
		});

		RequireNotLocked(Mutex);
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Inside Transaction Used In Nested Transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			MutexType Mutex;

			AutoRTFM::Testing::Abort([&]
			{
				UE::TScopeLock Lock(Mutex);
				AutoRTFM::CascadingAbortTransaction();
			});
		});

		AutoRTFM::Testing::Commit([&]
		{
			MutexType Mutex;

			AutoRTFM::Testing::Abort([&]
			{
				UE::TScopeLock Lock(Mutex);
				AutoRTFM::AbortTransaction();
			});
		});

		AutoRTFM::Testing::Abort([&]
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				UE::TScopeLock Lock(Mutex);
			});

			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				UE::TScopeLock Lock(Mutex);
			});
		});
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("In Static Local Initializer")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			MutexType Mutex;
		};

		auto Lambda = [&NumLocksUsed] AUTORTFM_ENABLE
		{
			static MyStruct Mine;

			// The above line above allocates a new lock, so update our count.
			NumLocksUsed = UE::Private::TransactionallySafeMutexPool::GetNumUsed();

			UE::TScopeLock _(Mine.Mutex);
			return 42;
		};

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(42 == Lambda());
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(42 == Lambda());

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(42 == Lambda());
		});

		REQUIRE(42 == Lambda());
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("In Static Local Initializer Called From Open")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			MutexType Mutex;
		};

		auto Lambda = [&NumLocksUsed] AUTORTFM_ENABLE
		{
			static MyStruct Mine;

			// The above line above allocates a new lock, so update our count.
			NumLocksUsed = UE::Private::TransactionallySafeMutexPool::GetNumUsed();

			UE::TScopeLock _(Mine.Mutex);
			return 42;
		};

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::Open([&] { REQUIRE(42 == Lambda()); });
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(42 == Lambda());

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&] { REQUIRE(42 == Lambda()); });
		});

		REQUIRE(42 == Lambda());
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Delete Heap Allocated Mutex")
	{
		SECTION("SingleThread") // Mutex owned and destructed by this thread
		{
			UE::FTransactionallySafeMutex* const Mutex = new UE::FTransactionallySafeMutex();

			AutoRTFM::Testing::Abort([&]
			{
				Mutex->Lock();
				delete Mutex;
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(!Mutex->IsLocked());

			AutoRTFM::Testing::Commit([&]
			{
				Mutex->Lock();
				delete Mutex;
			});
		}
			
		SECTION("MultiThread") // Mutex owned and destructed by another thread
		{
			// This test does not support retries due to coordination with another thread.
			AutoRTFMTestUtils::FScopedRetry NoRetry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

			enum EState
			{
				InitializeThread,
				MutexReady,
				MutexUsed,
				MutexDeleted,
			};

			struct AUTORTFM_OPEN FEvent
			{
				void Signal(EState State)
				{
					{
						std::unique_lock Lock(Mutex);
						CurrentState = State;
					}
					CV.notify_one();
				}

				void Wait(EState State)
				{
					std::unique_lock Lock(Mutex);
					CV.wait(Lock, [this, State] { return CurrentState == State; });
				}
			private:
				EState CurrentState = InitializeThread;
				std::mutex Mutex;
				std::condition_variable CV;
			};
				
			UE::FTransactionallySafeMutex* Mutex = nullptr;
			FEvent Event;

			std::thread Thread([&Mutex, &Event] AUTORTFM_OPEN
			{
				Mutex = new UE::FTransactionallySafeMutex();
				Event.Signal(MutexReady);
				Event.Wait(MutexUsed);
				delete Mutex;
				Event.Signal(MutexDeleted);
			});

			Event.Wait(MutexReady);

			SECTION("Lock, Abort, Destroy")
			{
				AutoRTFM::Testing::Abort([&]
				{
					Mutex->Lock();
					AutoRTFM::AbortTransaction();
				});

				REQUIRE(!Mutex->IsLocked());

				Event.Signal(MutexUsed);
				Event.Wait(MutexDeleted);
			}

			SECTION("Lock, Unlock, Destroy, Abort")
			{
				AutoRTFM::Testing::Abort([&]
				{
					Mutex->Lock();
					Mutex->Unlock();

					Event.Signal(MutexUsed);
					Event.Wait(MutexDeleted);

					AutoRTFM::AbortTransaction();
				});
			}

			SECTION("Lock, Unlock, Destroy, Commit")
			{
				AutoRTFM::Testing::Commit([&]
				{
					Mutex->Lock();
					Mutex->Unlock();

					Event.Signal(MutexUsed);
					Event.Wait(MutexDeleted);
				});
			}

			Thread.join();
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Lock Within, Unlock Outside")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				Mutex.Lock();
			});

			RequireLocked(Mutex);
			Mutex.Unlock();
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
	{
		SECTION("TryLock Within, Unlock Outside")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Mutex.TryLock());
			});

			RequireLocked(Mutex);
			Mutex.Unlock();
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Lock Outside, Unlock Within")
		{
			MutexType Mutex;
			Mutex.Lock();

			AutoRTFM::Testing::Commit([&]
			{
				RequireLocked(Mutex);
				Mutex.Unlock();
			});

			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Lock, Unlock, Lock")
		{
			MutexType Mutex;

			SECTION("Commit")
			{
				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Lock();
					Mutex.Unlock();
					Mutex.Lock();
				});

				RequireLocked(Mutex);
				Mutex.Unlock();
			}

			SECTION("Abort")
			{
				AutoRTFM::Testing::Abort([&]
				{
					Mutex.Lock();
					Mutex.Unlock();
					Mutex.Lock();
					AutoRTFM::AbortTransaction();
				});

				RequireNotLocked(Mutex);
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
	{
		SECTION("TryLock, Unlock, TryLock")
		{
			MutexType Mutex;

			SECTION("Commit")
			{
				AutoRTFM::Testing::Commit([&]
				{
					REQUIRE(Mutex.TryLock());
					Mutex.Unlock();
					REQUIRE(Mutex.TryLock());
				});

				RequireLocked(Mutex);
				Mutex.Unlock();
			}
				
			SECTION("Abort")
			{
				AutoRTFM::Testing::Abort([&]
				{
					REQUIRE(Mutex.TryLock());
					Mutex.Unlock();
					REQUIRE(Mutex.TryLock());
					AutoRTFM::AbortTransaction();
				});

				RequireNotLocked(Mutex);
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Unlock, Lock, Unlock")
		{
			MutexType Mutex;
			Mutex.Lock();

			SECTION("Commit")
			{
				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();
					Mutex.Lock();
					Mutex.Unlock();
				});

				RequireNotLocked(Mutex);
			}

			SECTION("Commit")
			{
				AutoRTFM::Testing::Abort([&]
				{
					Mutex.Unlock();
					Mutex.Lock();
					Mutex.Unlock();
					AutoRTFM::AbortTransaction();
				});

				RequireLocked(Mutex);
				Mutex.Unlock();
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
	{
		SECTION("Unlock, TryLock, Unlock")
		{
			MutexType Mutex;
			Mutex.Lock();

			SECTION("Commit")
			{
				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();
					REQUIRE(Mutex.TryLock());
					Mutex.Unlock();
				});

				RequireNotLocked(Mutex);
			}

			SECTION("Abort")
			{
				AutoRTFM::Testing::Abort([&]
				{
					Mutex.Unlock();
					REQUIRE(Mutex.TryLock());
					Mutex.Unlock();
					AutoRTFM::AbortTransaction();
				});

				RequireLocked(Mutex);
				Mutex.Unlock();
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Lock Outside, Unlock & Lock Within")
		{
			MutexType Mutex;
			Mutex.Lock();

			AutoRTFM::Testing::Commit([&]
			{
				RequireLocked(Mutex);
				Mutex.Unlock();
				Mutex.Lock();
			});

			RequireLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Commit(Lock, Commit(Unlock))")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();
				});

				RequireLocked(Mutex);
			});

			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Abort(Lock, Commit(Unlock, Lock))")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Abort([&]
			{
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();
					Mutex.Lock();
				});

				RequireLocked(Mutex);

				AutoRTFM::AbortTransaction();
			});

			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsUnbalancedLockingInTransaction)
	{
		SECTION("Contended Lock")
		{
			// These subtests do not support transactional retries.
			AUTORTFM_SCOPED_DISABLE_RETRY();

			SECTION("Contender Parks Transaction")
			{
				MutexType Mutex;
				std::atomic_uint Orderer = 0;

				std::thread Contender([&] AUTORTFM_OPEN
				{
					REQUIRE(0 == Orderer);
					Mutex.Lock();
					Orderer += 1; // unblock main thread
					while (1 == Orderer) {} // wait on main thread
					std::this_thread::sleep_for(std::chrono::milliseconds(100));
					Mutex.Unlock();
				});

				while (0 == Orderer) {} // wait on contender

				AutoRTFM::Testing::Commit([&]
				{
					if constexpr (bSupportsIsLocked)
					{
						REQUIRE(Mutex.IsLocked());
					}
					AutoRTFM::Open([&] { Orderer += 1; }); // unblock contender
					Mutex.Lock();
				});

				if constexpr (bSupportsIsLocked)
				{
					REQUIRE(Mutex.IsLocked());
				}
				Mutex.Unlock();
				Contender.join();
			}

			SECTION("Transaction Parks Contender")
			{
				MutexType Mutex;
				std::atomic_uint Orderer = 0;

				std::thread Contender([&] AUTORTFM_OPEN
				{
					while (0 == Orderer) {} // wait on main thread
					Mutex.Lock();
				});

				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Lock();
					AutoRTFM::Open([&]
					{
						Orderer += 1; // unblock contender
						std::this_thread::sleep_for(std::chrono::milliseconds(100));
					});
					Mutex.Unlock();
				});

				Contender.join();
				Mutex.Unlock();
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("In On-Commit")
	{
		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Transact(Lock, Unlock-On-Commit)")
			{
				MutexType Mutex;
				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Lock();

					AutoRTFM::OnCommit([&]
					{
						RequireLocked(Mutex);
						Mutex.Unlock();
						RequireNotLocked(Mutex);
					});
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Transact(Unlock-On-Commit, Lock)")
			{
				MutexType Mutex;
				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::OnCommit([&]
					{
						RequireLocked(Mutex);
						Mutex.Unlock();
						RequireLocked(Mutex);
					});
						
					Mutex.Lock();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Transact(TryLock, Unlock-On-Commit)")
			{
				MutexType Mutex;
				AutoRTFM::Testing::Commit([&]
				{
					REQUIRE(Mutex.TryLock());
					REQUIRE(!Mutex.TryLock());

					AutoRTFM::OnCommit([&]
					{
						RequireLocked(Mutex);
						Mutex.Unlock();
						RequireNotLocked(Mutex);
					});
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Transact(Unlock-On-Commit, TryLock)")
			{
				MutexType Mutex;
				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::OnCommit([&]
					{
						RequireLocked(Mutex);
						Mutex.Unlock();
						RequireLocked(Mutex);
					});
						
					REQUIRE(Mutex.TryLock());
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Lock, Transact(Lock-On-Commit, Unlock)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::OnCommit([&]
					{
						RequireLocked(Mutex);
						Mutex.Lock();
						RequireLocked(Mutex);
					});
						
					Mutex.Unlock();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Lock, Transact(Unlock, Lock-On-Commit)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();

					AutoRTFM::OnCommit([&]
					{
						RequireNotLocked(Mutex);
						Mutex.Lock();
						RequireLocked(Mutex);
					});
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Lock, Transact(Unlock, TryLock-On-Commit)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					Mutex.Unlock();

					AutoRTFM::OnCommit([&]
					{
						REQUIRE(Mutex.TryLock());
					});
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Lock, Transact(TryLock-On-Commit, Unlock)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Commit([&]
				{
					AutoRTFM::OnCommit([&]
					{
						REQUIRE(Mutex.TryLock());
					});

					Mutex.Unlock();
				});
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("In On-Abort")
	{
		SECTION("Transact(Check-On-Abort, Lock, Check-On-Abort, Abort)")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				AutoRTFM::OnAbort([&]
				{
					RequireNotLocked(Mutex);
				});

				Mutex.Lock();

				AutoRTFM::OnAbort([&]
				{
					RequireLocked(Mutex);
				});

				AutoRTFM::AbortTransaction();
			});
		}

		if constexpr (bSupportsTryLockInTransaction)
		{
			SECTION("Transact(Check-On-Abort, TryLock, Check-On-Abort, Abort)")
			{
				MutexType Mutex;
				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnAbort([&]
					{
						RequireNotLocked(Mutex);
					});
						
					REQUIRE(Mutex.TryLock());

					AutoRTFM::OnAbort([&]
					{
						RequireLocked(Mutex);
					});

					AutoRTFM::AbortTransaction();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Lock, Transact(Check-On-Abort, Unlock, Abort)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnAbort([&]
					{
						RequireLocked(Mutex);
					});
						
					Mutex.Unlock();

					AutoRTFM::AbortTransaction();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction)
		{
			SECTION("Lock, Transact(Unlock, Check-On-Abort, Abort)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Abort([&]
				{
					Mutex.Unlock();

					AutoRTFM::OnAbort([&]
					{
						RequireLocked(Mutex);
					});

					AutoRTFM::AbortTransaction();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Lock, Transact(Unlock, TryLock-On-Abort, Abort)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Abort([&]
				{
					Mutex.Unlock();

					AutoRTFM::OnAbort([&]
					{
						REQUIRE(!Mutex.TryLock());
					});

					AutoRTFM::AbortTransaction();
				});
			}
		}

		if constexpr (bSupportsUnbalancedLockingInTransaction && bSupportsTryLockInTransaction)
		{
			SECTION("Lock, Transact(TryLock-On-Abort, Unlock, Abort)")
			{
				MutexType Mutex;
				Mutex.Lock();

				AutoRTFM::Testing::Abort([&]
				{
					AutoRTFM::OnAbort([&]
					{
						REQUIRE(!Mutex.TryLock());
					});

					Mutex.Unlock();

					AutoRTFM::AbortTransaction();
				});

				RequireLocked(Mutex);
			}
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Locked Mutex In Destructed Object")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			MutexType Mutex;

			~MyStruct()
			{
				// This zeroes out the mutex's state pointer, causing it to leak; real code shouldn't do this!
				memset(this, 0, sizeof(MyStruct));
			}
		};

		std::unique_ptr<MyStruct> Mine(new MyStruct);

		AutoRTFM::Testing::Commit([&]
		{
			Mine->Mutex.Lock();
			Mine.reset();
			REQUIRE(!Mine);
		});

		// The lock inside MyStruct is now permanently leaked. We account for this pool size increase here.
		NumLocksUsed = UE::Private::TransactionallySafeMutexPool::GetNumUsed();
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsMixedOpenCloseLocking)
	{
		SECTION("Closed Lock Then Open Unlock")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				Mutex.Lock();
				RequireLocked(Mutex);

				AutoRTFM::Open([&]
				{
					Mutex.Unlock();
				});
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsMixedOpenCloseLocking && bSupportsTryLockInTransaction)
	{
		SECTION("Closed TryLock Then Open Unlock")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Mutex.TryLock());
				RequireLocked(Mutex);

				AutoRTFM::Open([&]
				{
					Mutex.Unlock();
				});
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsMixedOpenCloseLocking)
	{
		SECTION("Open Lock Then Closed Unlock")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Open([&]
				{
					Mutex.Lock();
				});
				RequireLocked(Mutex);

				Mutex.Unlock();
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bSupportsMixedOpenCloseLocking && bSupportsTryLockInTransaction)
	{
		SECTION("Open TryLock Then Closed Unlock")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				AutoRTFM::Open([&]
				{
					REQUIRE(Mutex.TryLock());
				});
				RequireLocked(Mutex);

				Mutex.Unlock();
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsRecursive)
	{
		SECTION("Transact(Lock(Lock()))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				RequireNotLocked(Mutex);
				{
					UE::TScopeLock LockA(Mutex);
					RequireLocked(Mutex);
					{
						UE::TScopeLock LockB(Mutex);
						RequireLocked(Mutex);
					}
					RequireLocked(Mutex);
				}
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsRecursive)
	{
		SECTION("Lock(Lock())")
		{
			MutexType Mutex;
			RequireNotLocked(Mutex);
			{
				UE::TScopeLock LockA(Mutex);
				RequireLocked(Mutex);
				{
					UE::TScopeLock LockB(Mutex);
					RequireLocked(Mutex);
				}
			}
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsRecursive)
	{
		SECTION("Transact(Lock(Lock(Abort)))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				RequireNotLocked(Mutex);
				{
					UE::TScopeLock LockA(Mutex);
					RequireLocked(Mutex);
					{
						UE::TScopeLock LockB(Mutex);
						RequireLocked(Mutex);
						AutoRTFM::AbortTransaction();
					}
				}
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsRecursive)
	{
		SECTION("Transact(Lock(Lock(), Abort))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				RequireNotLocked(Mutex);
				{
					UE::TScopeLock LockA(Mutex);
					RequireLocked(Mutex);
					{
						UE::TScopeLock LockB(Mutex);
						RequireLocked(Mutex);
					}
					AutoRTFM::AbortTransaction();
				}
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsRecursive)
	{
		SECTION("Transact(Lock(Transact(Lock(), Abort)))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				RequireNotLocked(Mutex);
				UE::TScopeLock LockA(Mutex);
				RequireLocked(Mutex);
				AutoRTFM::Testing::Abort([&]
				{
					UE::TScopeLock LockB(Mutex);
					RequireLocked(Mutex);
					AutoRTFM::AbortTransaction();
				});
				RequireLocked(Mutex);
			});
			RequireNotLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("SharedLock()")
		{
			MutexType Mutex;
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
			{
				UE::TSharedLock LockA(Mutex);
				RequireNotLocked(Mutex);
				RequireLockShared(Mutex);
			}
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("TryLockShared, UnlockShared")
		{
			MutexType Mutex;
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);

			UE::Core::Private::FSharedRecursiveMutexLink Link;
			REQUIRE(TryLockShared(Mutex, Link));
			RequireNotLocked(Mutex);
			RequireLockShared(Mutex);

			UnlockShared(Mutex, Link);
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("Commit(TryLockShared, UnlockShared)")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				RequireNotLocked(Mutex);
				RequireNotSharedLocked(Mutex);

				UE::Core::Private::FSharedRecursiveMutexLink Link;
				REQUIRE(TryLockShared(Mutex, Link));
				RequireLocked(Mutex);  // a transactional LockShared is pessimized to a Lock

				UnlockShared(Mutex, Link);
				RequireLocked(Mutex);  // a transactional lock is held until the transaction completes
			});

			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("Abort(TryLockShared)")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Abort([&]
			{
				RequireNotLocked(Mutex);
				RequireNotSharedLocked(Mutex);

				UE::Core::Private::FSharedRecursiveMutexLink Link;
				REQUIRE(TryLockShared(Mutex, Link));
				RequireLocked(Mutex);  // a transactional LockShared is pessimized to a Lock

				AutoRTFM::AbortTransaction();
			});

			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("Lock, TryLockShared, Unlock")
		{
			MutexType Mutex;
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);

			{
				UE::TScopeLock Lock(Mutex);
				RequireLocked(Mutex);

				UE::Core::Private::FSharedRecursiveMutexLink Link;
				REQUIRE(!TryLockShared(Mutex, Link));
				RequireLocked(Mutex);
			}

			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("Commit(Lock, TryLockShared, Unlock)")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Commit([&]
			{
				RequireNotLocked(Mutex);
				RequireNotSharedLocked(Mutex);

				{
					UE::TScopeLock Lock(Mutex);
					RequireLocked(Mutex);

					UE::Core::Private::FSharedRecursiveMutexLink Link;
					REQUIRE(!TryLockShared(Mutex, Link));
					RequireLocked(Mutex);
				}
			});

			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared)
	{
		SECTION("Abort(Lock, TryLockShared)")
		{
			MutexType Mutex;

			AutoRTFM::Testing::Abort([&]
			{
				RequireNotLocked(Mutex);
				RequireNotSharedLocked(Mutex);

				{
					UE::TScopeLock Lock(Mutex);
					RequireLocked(Mutex);

					UE::Core::Private::FSharedRecursiveMutexLink Link;
					REQUIRE(!TryLockShared(Mutex, Link));
					RequireLocked(Mutex);

					AutoRTFM::AbortTransaction();
				}
			});

			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared && bIsRecursive)
	{
		SECTION("SharedLock(SharedLock())")
		{
			MutexType Mutex;
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
			{
				UE::TSharedLock LockA(Mutex);
				RequireNotLocked(Mutex);
				RequireLockShared(Mutex);
				{
					UE::TSharedLock LockB(Mutex);
					RequireNotLocked(Mutex);
					RequireLockShared(Mutex);
				}
				RequireNotLocked(Mutex);
				RequireLockShared(Mutex);
			}
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared && bIsRecursive)
	{
		SECTION("Transact(SharedLock(SharedLock()))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				RequireNotSharedLocked(Mutex);
				{
					UE::TSharedLock LockA(Mutex);
					RequireLocked(Mutex);
					RequireNotSharedLocked(Mutex);
					{
						UE::TSharedLock LockB(Mutex);
						RequireLocked(Mutex);
						RequireNotSharedLocked(Mutex);
					}
					RequireLocked(Mutex);
					RequireNotSharedLocked(Mutex);
				}
			});
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared && bIsRecursive)
	{
		SECTION("Transact(SharedLock(SharedLock(Abort)))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				RequireNotSharedLocked(Mutex);
				{
					UE::TSharedLock LockA(Mutex);
					RequireLocked(Mutex);
					RequireNotSharedLocked(Mutex);
					{
						UE::TSharedLock LockB(Mutex);
						RequireLocked(Mutex);
						RequireNotSharedLocked(Mutex);
						AutoRTFM::AbortTransaction();
					}
				}
			});
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared && bIsRecursive)
	{
		SECTION("Transact(SharedLock(SharedLock(), Abort))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				RequireNotSharedLocked(Mutex);
				{
					UE::TSharedLock LockA(Mutex);
					RequireLocked(Mutex);
					RequireNotSharedLocked(Mutex);
					{
						UE::TSharedLock LockB(Mutex);
						RequireLocked(Mutex);
						RequireNotSharedLocked(Mutex);
					}
					AutoRTFM::AbortTransaction();
				}
			});
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Retry Releases Locks")
	{
		MutexType Mutex;
		bool bAlreadyRetried = false;

		AutoRTFM::Testing::Commit([&]
		{
			// Mutex must start off unlocked.
			RequireNotLocked(Mutex);
			UE::TScopeLock Lock(Mutex);
			RequireLocked(Mutex);

			if (!bAlreadyRetried)
			{
				AutoRTFM::OnRetry([&]
				{
					// Mutex must be unlocked when we reach OnRetry, as OnRetry happens after all OnAborts.
					RequireNotLocked(Mutex);
					bAlreadyRetried = true;
				});

				// Trigger a retry with the lock held.
				AutoRTFM::CascadingRetryTransaction();
			}
		});
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	if constexpr (bIsShared && bIsRecursive)
	{
		SECTION("Transact(SharedLock(Transact(SharedLock(), Abort)))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				RequireNotSharedLocked(Mutex);
				UE::TSharedLock LockA(Mutex);
				RequireLocked(Mutex);
				RequireNotSharedLocked(Mutex);
				AutoRTFM::Testing::Abort([&]
				{
					UE::TSharedLock LockB(Mutex);
					RequireLocked(Mutex);
					RequireNotSharedLocked(Mutex);
					AutoRTFM::AbortTransaction();
				});
				RequireLocked(Mutex);
				RequireNotSharedLocked(Mutex);
			});
			RequireNotLocked(Mutex);
			RequireNotSharedLocked(Mutex);
		}
	}

	REQUIRE(NumLocksUsed == UE::Private::TransactionallySafeMutexPool::GetNumUsed());

	SECTION("Benchmarks")
	{
		constexpr int Iterations = 10000;

		BENCHMARK("Commit: Loop(Transact(Ctor, Lock, Unlock, Dtor))")
		{
			for (int Index = 0; Index < Iterations; ++Index)
			{
				AutoRTFM::Testing::Commit([&]
				{
					MutexType Mutex;
					Mutex.Lock();
					Mutex.Unlock();
				});
			}
		};

		BENCHMARK("Commit: Transact(Loop(Ctor, Lock, Unlock, Dtor))")
		{
			AutoRTFM::Testing::Commit([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					MutexType Mutex;
					Mutex.Lock();
					Mutex.Unlock();
				}
			});
		};

		BENCHMARK("Commit: Transact(Loop(Lock, Unlock))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Commit([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex.Lock();
					Mutex.Unlock();
				}
			});
		};

		BENCHMARK("Commit: Transact(Loop(Lock), Loop(Unlock))")
		{
			MutexType Mutex[Iterations];
			AutoRTFM::Testing::Commit([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex[Index].Lock();
				}
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex[Index].Unlock();
				}
			});
		};

		BENCHMARK("Abort: Loop(Transact(Ctor, Lock, Unlock, Dtor))")
		{
			for (int Index = 0; Index < Iterations; ++Index)
			{
				AutoRTFM::Testing::Abort([&]
				{
					MutexType Mutex;
					Mutex.Lock();
					Mutex.Unlock();
					AutoRTFM::AbortTransaction();
				});
			}
		};

		BENCHMARK("Abort: Transact(Loop(Ctor, Lock, Unlock, Dtor))")
		{
			AutoRTFM::Testing::Abort([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					MutexType Mutex;
					Mutex.Lock();
					Mutex.Unlock();
				}
				AutoRTFM::AbortTransaction();
			});
		};

		BENCHMARK("Abort: Transact(Loop(Lock, Unlock))")
		{
			MutexType Mutex;
			AutoRTFM::Testing::Abort([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex.Lock();
					Mutex.Unlock();
				}
				AutoRTFM::AbortTransaction();
			});
		};

		BENCHMARK("Abort: Transact(Loop(Lock), Loop(Unlock))")
		{
			MutexType Mutex[Iterations];
			AutoRTFM::Testing::Abort([&]
			{
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex[Index].Lock();
				}
				for (int Index = 0; Index < Iterations; ++Index)
				{
					Mutex[Index].Unlock();
				}
				AutoRTFM::AbortTransaction();
			});
		};
	}
}
