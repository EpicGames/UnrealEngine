// Copyright Epic Games, Inc. All Rights Reserved.

#include "MyAutoRTFMTestObject.h"

UMyAutoRTFMTestObject::FConstructorCallback* UMyAutoRTFMTestObject::ConstructorCallback = nullptr;

UMyAutoRTFMTestObject::UMyAutoRTFMTestObject(const FObjectInitializer& ObjectInitializer /* = FObjectInitializer::Get() */) : Value(42)
{
	UObject* const Obj = ObjectInitializer.GetObj();
	UObject* const Outer = Obj->GetOuter();

	if (Outer->IsA<UMyAutoRTFMTestObject>())
	{
		UMyAutoRTFMTestObject* const OuterAsType = static_cast<UMyAutoRTFMTestObject*>(Outer);
		OuterAsType->Value += 13;
	}

	if (ConstructorCallback)
	{
		ConstructorCallback(ObjectInitializer, *this);
	}
}
