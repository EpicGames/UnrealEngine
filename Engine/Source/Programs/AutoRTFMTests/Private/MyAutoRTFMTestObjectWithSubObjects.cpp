// Copyright Epic Games, Inc. All Rights Reserved.

#include "MyAutoRTFMTestObjectWithSubObjects.h"
#include "MyAutoRTFMTestObject.h"

FName UMyAutoRTFMTestObjectWithSubObjects::GetSubObjectName()
{
	static FName Name(FString::Printf(TEXT("%s%u"), TEXT(__FILE__), __LINE__));
	return Name;
}

UMyAutoRTFMTestObjectWithSubObjects::UMyAutoRTFMTestObjectWithSubObjects(const FObjectInitializer& ObjectInitializer /* = FObjectInitializer::Get() */)
{
	// Having an object that the default subobject replaces pushes us down a new and exciting codepath.
	UMyAutoRTFMTestObject* const Replacee = NewObject<UMyAutoRTFMTestObject>(this, UMyAutoRTFMTestObjectWithSubObjects::GetSubObjectName());

	SubObject = CreateDefaultSubobject<UMyAutoRTFMTestObject>(GetSubObjectName());
}
