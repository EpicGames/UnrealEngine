// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectLifetimeHelper.h"
#include "Catch2Includes.h"

namespace AutoRTFMTestUtils
{

FObjectLifetimeHelper::FObjectLifetimeHelper()
{
	ConstructorCalls++;
}

FObjectLifetimeHelper::~FObjectLifetimeHelper()
{
	DestructorCalls++;
}

FObjectLifetimeHelper::FObjectLifetimeHelper(int Value) : Value{Value}
{
	ConstructorCalls++;
}

FObjectLifetimeHelper::FObjectLifetimeHelper(const FObjectLifetimeHelper& Other) : Value{Other.Value}, bIsValid(Other.bIsValid)
{
	ConstructorCalls++;
	REQUIRE(bIsValid);
}

FObjectLifetimeHelper::FObjectLifetimeHelper(FObjectLifetimeHelper&& Other) : Value{Other.Value}, bIsValid(Other.bIsValid)
{
	Other.Value = 0;
	Other.bIsValid = false;
	ConstructorCalls++;
	REQUIRE(bIsValid);
}

FObjectLifetimeHelper& FObjectLifetimeHelper::operator = (const FObjectLifetimeHelper& Other)
{
	Value = Other.Value;
	bIsValid = Other.bIsValid;
	REQUIRE(bIsValid);
	return *this;
}

FObjectLifetimeHelper& FObjectLifetimeHelper::operator = (FObjectLifetimeHelper&& Other)
{
	Value = Other.Value;
	bIsValid = Other.bIsValid;
	Other.Value = 0;
	Other.bIsValid = false;
	REQUIRE(bIsValid);
	return *this;
}

bool FObjectLifetimeHelper::operator == (FObjectLifetimeHelper Other) const
{
	REQUIRE(bIsValid);
	REQUIRE(Other.bIsValid);
	return Value == Other.Value;
}

size_t FObjectLifetimeHelper::ConstructorCalls = 0;
size_t FObjectLifetimeHelper::DestructorCalls = 0;

} // namespace FObjectLifetimeHelper
