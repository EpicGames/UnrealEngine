// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstddef>
#include <functional>

namespace AutoRTFMTestUtils
{
	
// A helper type that acts like an int, but also tracks number of ctors, dtors
// and whether its used after move.
struct FObjectLifetimeHelper
{
	FObjectLifetimeHelper();
	~FObjectLifetimeHelper();
	FObjectLifetimeHelper(int Value);
	FObjectLifetimeHelper(const FObjectLifetimeHelper& Other);
	FObjectLifetimeHelper(FObjectLifetimeHelper&& Other);
	FObjectLifetimeHelper& operator = (const FObjectLifetimeHelper& Other);
	FObjectLifetimeHelper& operator = (FObjectLifetimeHelper&& Other);
	bool operator == (FObjectLifetimeHelper Other) const;

	int Value = 0;
	bool bIsValid = true;
	static size_t ConstructorCalls;
	static size_t DestructorCalls;
};

}  // namespace AutoRTFMTestUtils

// std::hash specialization for AutoRTFMTestUtils::FObjectLifetimeHelper
namespace std
{
	template<>
	class hash<AutoRTFMTestUtils::FObjectLifetimeHelper>
	{
	public:
		size_t operator()(const AutoRTFMTestUtils::FObjectLifetimeHelper& Object) const
		{
			return std::hash<int>{}(Object.Value);
		}
	};
}
