// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"

#include <atomic>
#include <catch_amalgamated.hpp>
#include <map>
#include <vector>

namespace
{

UE_AUTORTFM_ALWAYS_OPEN static void AssignIntPointer(int* Pointer, int Value, int* LineThatPerformedAssignment = nullptr)
{
    if (LineThatPerformedAssignment) { *LineThatPerformedAssignment = __LINE__ + 1; }
    *Pointer = Value;
}

using FnPtr = void (*)(int* Pointer);

UE_AUTORTFM_ALWAYS_OPEN static FnPtr GetFunctionPointer()
{
	auto FnPtr = +[](int* Pointer)
	{
		*Pointer = 42;
	};
	return FnPtr;
}

}

TEST_CASE("Open")
{
    bool bDidRun = false;
    REQUIRE(
        AutoRTFM::ETransactionResult::Committed ==
        AutoRTFM::Transact([&]
        {
            AutoRTFM::Open([&] { bDidRun = true; });
            AutoRTFM::Open([&] { REQUIRE(bDidRun); });
        }));
    REQUIRE(bDidRun);
}

TEST_CASE("Open.Large")
{
    int x = 42;
    std::vector<int> v;
    std::map<int, std::vector<int>> m;
    bool bRanOpen = false;
    v.push_back(100);
    m[1].push_back(2);
    m[1].push_back(3);
    m[4].push_back(5);
    m[6].push_back(7);
    m[6].push_back(8);
    m[6].push_back(9);
    REQUIRE(
        AutoRTFM::ETransactionResult::Committed ==
        AutoRTFM::Transact([&] {
            x = 5;

            for (size_t n = 10; n--;)
            {
                v.push_back(2 * n);
            }

            m.clear();
            m[10].push_back(11);
            m[12].push_back(13);
            m[12].push_back(14);

            AutoRTFM::Open([&] {
#if 0
                // These checks are UB, because the open is interacting with transactional data!
                REQUIRE(x == 42);
                REQUIRE(v.size() == 1);
                REQUIRE(v[0] == 100);
                REQUIRE(m.size() == 3);
                REQUIRE(m[1].size() == 2);
                REQUIRE(m[1][0] == 2);
                REQUIRE(m[1][1] == 3);
                REQUIRE(m[4].size() == 1);
                REQUIRE(m[4][0] == 5);
                REQUIRE(m[6].size() == 3);
                REQUIRE(m[6][0] == 7);
                REQUIRE(m[6][1] == 8);
                REQUIRE(m[6][2] == 9);
#endif
                bRanOpen = true;
            });
        }));

    REQUIRE(bRanOpen);
    REQUIRE(x == 5);
    REQUIRE(v.size() == 11);
    REQUIRE(v[0] == 100);
    REQUIRE(v[1] == 18);
    REQUIRE(v[2] == 16);
    REQUIRE(v[3] == 14);
    REQUIRE(v[4] == 12);
    REQUIRE(v[5] == 10);
    REQUIRE(v[6] == 8);
    REQUIRE(v[7] == 6);
    REQUIRE(v[8] == 4);
    REQUIRE(v[9] == 2);
    REQUIRE(v[10] == 0);
    REQUIRE(m.size() == 2);
    REQUIRE(m[10].size() == 1);
    REQUIRE(m[10][0] == 11);
    REQUIRE(m[12].size() == 2);
    REQUIRE(m[12][0] == 13);
    REQUIRE(m[12][1] == 14);
}

TEST_CASE("Open.Atomics")
{
    std::atomic<bool> bDidRun = false;
    REQUIRE(
        AutoRTFM::ETransactionResult::Committed ==
        AutoRTFM::Transact([&]
        {
            AutoRTFM::Open([&] { bDidRun = true; });
            AutoRTFM::Open([&] { REQUIRE(bDidRun); });
        }));
    REQUIRE(bDidRun);
}

TEST_CASE("Open.FunctionPtrFromAlwaysOpenFunction")
{
	FnPtr Func = GetFunctionPointer();

	int Value = 0;

	SECTION("Commit")
	{
		AutoRTFM::Testing::Commit([&]
		{
			Func(&Value);
			REQUIRE(Value == 42);
		});
		REQUIRE(Value == 42);
	}

	SECTION("Abort")
	{
		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Func(&Value);
			REQUIRE(Value == 42);
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(Result == AutoRTFM::ETransactionResult::AbortedByRequest);
		REQUIRE(Value == 0);
	}

	SECTION("Open")
	{
		AutoRTFM::Open([&]
		{
			Func(&Value);
			REQUIRE(Value == 42);
		});
		REQUIRE(Value == 42);
	}
}

struct FReturnFromOpenHelperFlags
{
    struct FMode
    {
        bool bConstructed = false;
        bool bCopyConstructed = false;
        bool bMoveConstructed = false;
    };
    
    FMode Open;
    FMode Closed;
    
    FMode* operator->()
    {
        if (AutoRTFM::IsClosed())
        {
            return &Closed;
        }
        else
        {
            return &Open;
        }
    }
};

template<AutoRTFM::EReturnFromOpenMode Mode>
struct TReturnFromOpenHelper
{
    static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = Mode;

    TReturnFromOpenHelper(FReturnFromOpenHelperFlags& Flags, int Value) : Flags{Flags}, Value{Value}
    {
        Flags->bConstructed = true;
    }

    TReturnFromOpenHelper(const TReturnFromOpenHelper& Other) : Flags{Other.Flags}, Value{Other.Value}
    {
        Flags->bCopyConstructed = true;
    }

    TReturnFromOpenHelper(TReturnFromOpenHelper&& Other) : Flags{Other.Flags}, Value{Other.Value}
    {
        Flags->bMoveConstructed = true;
    }

    TReturnFromOpenHelper& operator = (const TReturnFromOpenHelper& Other) = delete;
    TReturnFromOpenHelper& operator = (TReturnFromOpenHelper&& Other) = delete;

    FReturnFromOpenHelperFlags& Flags;
    int const Value;
};

TEST_CASE("Open.ReturnValue")
{
    static_assert(AutoRTFM::ReturnFromOpenModeFor<int> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<float> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<int*> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<void> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<std::string> == AutoRTFM::EReturnFromOpenMode::Unsupported);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<TArray<std::string>> == AutoRTFM::EReturnFromOpenMode::Unsupported);
    static_assert(AutoRTFM::ReturnFromOpenModeFor<TOptional<std::string>> == AutoRTFM::EReturnFromOpenMode::Unsupported);

    SECTION("int")
    {
        int Value = 10;
        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
            {
                int Got = AutoRTFM::Open([] { return 42; });
                AutoRTFM::Open([&] { Value = Got; });
            });
        REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
        REQUIRE(42 == Value);
    }

    SECTION("TCHAR*")
    {
        FString Value = TEXT("<unassigned>");
        AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
        {
            // Note: AutoRTFM::Open() is returning a TCHAR*
            FString Got = AutoRTFM::Open([] { return TEXT("meow"); });
            AutoRTFM::Open([&] { Value = Got; });
        });
        REQUIRE(AutoRTFM::ETransactionResult::Committed == Result);
        REQUIRE(Value == TEXT("meow"));
    }

    SECTION("Custom Mode")
    {
        FReturnFromOpenHelperFlags Flags;
        
        SECTION("CopyConstructInClosed")
        {
            using Helper = TReturnFromOpenHelper<AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed>;
            static_assert(AutoRTFM::ReturnFromOpenModeFor<Helper> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
            AutoRTFM::Testing::Commit([&]
            {
                Helper Closed = AutoRTFM::Open([&] { return Helper{Flags, 42}; });
                REQUIRE(42 == Closed.Value);
                REQUIRE(Flags.Open.bConstructed);
                REQUIRE(!Flags.Open.bCopyConstructed);
                REQUIRE(!Flags.Open.bMoveConstructed);
                REQUIRE(!Flags.Closed.bConstructed);
                REQUIRE(Flags.Closed.bCopyConstructed);
                REQUIRE(!Flags.Closed.bMoveConstructed);
            });
        }

        SECTION("MoveConstructInClosed")
        {
            using Helper = TReturnFromOpenHelper<AutoRTFM::EReturnFromOpenMode::MoveConstructInClosed>;
            static_assert(AutoRTFM::ReturnFromOpenModeFor<Helper> == AutoRTFM::EReturnFromOpenMode::MoveConstructInClosed);
            AutoRTFM::Testing::Commit([&]
            {
                Helper Closed = AutoRTFM::Open([&] { return Helper{Flags, 42}; });
                REQUIRE(42 == Closed.Value);
                REQUIRE(Flags.Open.bConstructed);
                REQUIRE(!Flags.Open.bCopyConstructed);
                REQUIRE(!Flags.Open.bMoveConstructed);
                REQUIRE(!Flags.Closed.bConstructed);
                REQUIRE(!Flags.Closed.bCopyConstructed);
                REQUIRE(Flags.Closed.bMoveConstructed);
            });
        }
    }

    SECTION("Custom Method")
    {
        SECTION("AutoRTFMAssignFromOpenToClosed() by value")
        {
            struct AUTORTFM_ENABLE FMyStruct
            {
                int Value = 0;
                static void AutoRTFMAssignFromOpenToClosed(FMyStruct& Closed, FMyStruct Open)
                {
                    Closed.Value = Open.Value;
                }
            };
            static_assert(AutoRTFM::ReturnFromOpenModeFor<FMyStruct> == AutoRTFM::EReturnFromOpenMode::CustomMethod);

            AutoRTFM::Testing::Commit([&]
            {
                FMyStruct Closed = AutoRTFM::Open([] { return FMyStruct{42}; });
                REQUIRE(42 == Closed.Value);
            });
        }
        SECTION("AutoRTFMAssignFromOpenToClosed() by const-ref")
        {
            struct AUTORTFM_ENABLE FMyStruct
            {
                int Value = 0;
                static void AutoRTFMAssignFromOpenToClosed(FMyStruct& Closed, const FMyStruct& Open)
                {
                    Closed.Value = Open.Value;
                }
            };
            static_assert(AutoRTFM::ReturnFromOpenModeFor<FMyStruct> == AutoRTFM::EReturnFromOpenMode::CustomMethod);

            AutoRTFM::Testing::Commit([&]
            {
                FMyStruct Closed = AutoRTFM::Open([] { return FMyStruct{42}; });
                REQUIRE(42 == Closed.Value);
            });
        }
        SECTION("AutoRTFMAssignFromOpenToClosed() by rvalue-ref")
        {
            struct AUTORTFM_ENABLE FMyStruct
            {
                int Value = 0;
                bool* WasMoved = nullptr;
                static void AutoRTFMAssignFromOpenToClosed(FMyStruct& Closed, FMyStruct&& Open)
                {
                    Closed.Value = Open.Value;
                    *Open.WasMoved = true;
                }
            };
            static_assert(AutoRTFM::ReturnFromOpenModeFor<FMyStruct> == AutoRTFM::EReturnFromOpenMode::CustomMethod);

            AutoRTFM::Testing::Commit([&]
            {
                bool WasMoved = false;
                FMyStruct Closed = AutoRTFM::Open([&] { return FMyStruct{42, &WasMoved}; });
                REQUIRE(true == WasMoved);
                REQUIRE(42 == Closed.Value);
            });
        }
    }

    SECTION("TArray<FString>")
    {
        static_assert(AutoRTFM::ReturnFromOpenModeFor<TArray<FString>> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);

        AutoRTFM::Testing::Commit([&]
        {
            TArray<FString> Array = AutoRTFM::Open([]
            { 
                return TArray{FString{"Hello"}, FString{"World"}};
            });
            REQUIRE(Array.Num() == 2);
            REQUIRE(Array[0] == FString{"Hello"});
            REQUIRE(Array[1] == FString{"World"});
        });
    }

    SECTION("TOptional<int>")
    {
        static_assert(AutoRTFM::ReturnFromOpenModeFor<TOptional<int>> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);

        AutoRTFM::Testing::Commit([&]
        {
            TOptional<int> Optional = AutoRTFM::Open([]
            { 
                return TOptional{42};
            });
            REQUIRE(Optional.IsSet());
            REQUIRE(Optional == 42);
        });
    }

    SECTION("TOptional<TArray<FString>>")
    {
        static_assert(AutoRTFM::ReturnFromOpenModeFor<TOptional<TArray<FString>>> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed);
        
        AutoRTFM::Testing::Commit([&]
        {
            TOptional<TArray<FString>> Optional = AutoRTFM::Open([]
            { 
                return TArray{FString{"Hello"}, FString{"World"}};
            });
            REQUIRE(Optional.IsSet());
            REQUIRE(Optional->Num() == 2);
            REQUIRE(Optional.GetValue()[0] == FString{"Hello"});
            REQUIRE(Optional.GetValue()[1] == FString{"World"});
        });
    }
}

#if AUTORTFM_SANITIZER
TEST_CASE("Open.Collision")
{
    AUTORTFM_SCOPED_SANITIZER_MODE_WARN();

    struct FLargeStruct
    {
        int V[256];
    };

    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;
    int I = 0;
    int J = 0;
    int SafeToAssignTo = 0;
    int LineThatPerformsConflictingWrite = 0;

#define WRITE_THAT_CAUSES_COLLISION(Statement) Statement; LineThatPerformsConflictingWrite = __LINE__

    SECTION("NoCollision")
    {
        SECTION("Different Memory Locations")
        {
            AutoRTFM::Transact([&]
            {
                I = 42;
                AutoRTFM::Open([&] { J = 24; });
            });

            REQUIRE(I == 42);
            REQUIRE(J == 24);
        }

        SECTION("Transact(Open(RecordOpenWrite()), Open(Assign))")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&I);
                });
                AutoRTFM::Open([&]
                {
                    I = 42;
                });
            });

            REQUIRE(I == 42);
        }

        SECTION("Transact(Transact(Open(RecordOpenWrite())), Open(Assign))")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Transact([&]
                {
                    AutoRTFM::Open([&]
                    {
                        AutoRTFM::RecordOpenWrite(&I);
                    });
                });

                AutoRTFM::Open([&]
                {
                    I = 42;
                });
            });

            REQUIRE(I == 42);
        }

        SECTION("Transact(Transact(Open(RecordOpenWrite())), Open(RecordOpenWrite()), Open(Assign))")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Transact([&]
                {
                    AutoRTFM::Open([&]
                    {
                        AutoRTFM::RecordOpenWrite(&I);
                    });
                });
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&I);
                });
                AutoRTFM::Open([&]
                {
                    I = 24;
                });
            });

            REQUIRE(I == 24);
        }

        SECTION("Transact(Open(RecordOpenWrite()), Open(Assign)) <large>")
        {
            FLargeStruct LargeStruct{};
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&LargeStruct);
                });
                AutoRTFM::Open([&]
                {
                    for (int& V : LargeStruct.V)
                    {
                        V = 42;
                    }
                });
            });

            for (int V : LargeStruct.V)
            {
                REQUIRE(V == 42);
            }

        }

        SECTION("Transact(Open(RecordOpenWrite()), Open(RecordOpenWrite()), Open(Assign))")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&I);
                });
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&I);
                });
                AutoRTFM::Open([&]
                {
                    I = 24;
                });
            });

        }

        SECTION("Transact(Open(RecordOpenWrite()), Open(RecordOpenWrite()), Open(Assign)) <large>")
        {
            FLargeStruct LargeStruct{};
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&LargeStruct);
                });
                AutoRTFM::Open([&]
                {
                    AutoRTFM::RecordOpenWrite(&LargeStruct);
                });
                AutoRTFM::Open([&]
                {
                    SafeToAssignTo = 10;

                    LargeStruct.V[255] = 42;

                    SafeToAssignTo = 20;
                });
            });

        }

        SECTION("Transact(Open(Assign), Assign)")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&] { I = 24; });
                I = 42;
            });

            REQUIRE(I == 42);
        }

        SECTION("Transact(Assign, Open(Transact(Assign)))")
        {
            AutoRTFM::Transact([&]
            {
                I = 24;
                AutoRTFM::Open([&]
                {
                    AutoRTFM::Transact([&]
                    {
                        I = 42;
                    });
                });
            });

            REQUIRE(I == 42);
        }

        SECTION("Transact(Open(Transact(Assign)), Assign)")
        {
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    AutoRTFM::Transact([&]
                    {
                        I = 24;
                    });
                });
                I = 42;
            });

            REQUIRE(I == 42);
        }
    }

    SECTION("Transact(Assign, Open(Assign))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;

                WRITE_THAT_CAUSES_COLLISION(I = 24);

                SafeToAssignTo = 20;
            });
        });
    }

    SECTION("Transact(Assign, AUTORTFM_SANITIZER_DISABLE_SCOPE(Open(Assign)))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;

            AUTORTFM_SANITIZER_DISABLE_SCOPE();

            AutoRTFM::Open([&]
            {
                I = 24; // Sanitizer error silenced by AUTORTFM_SANITIZER_DISABLE_SCOPE()
            });
        });
    }

    SECTION("Transact(Assign, Open(AUTORTFM_SANITIZER_DISABLE_SCOPE(Assign)))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;

            AutoRTFM::Open([&]
            {
                AUTORTFM_SANITIZER_DISABLE_SCOPE();

                I = 24; // Sanitizer error silenced by AUTORTFM_SANITIZER_DISABLE_SCOPE()
            });
        });
    }

    SECTION("Transact(Assign, Transact(Open(Assign)))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Open([&]
                {
                    SafeToAssignTo = 10;
                    
                    WRITE_THAT_CAUSES_COLLISION(I = 24);

                    SafeToAssignTo = 20;
                });
            });
        });
    }

    SECTION("Transact(Assign, Transact(Transact(Open(Assign))))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Transact([&]
            {
                AutoRTFM::Transact([&]
                {
                    AutoRTFM::Open([&]
                    {
                        SafeToAssignTo = 10;

                        WRITE_THAT_CAUSES_COLLISION(I = 24);

                        SafeToAssignTo = 20;
                    });
                });
            });
        });
    }

    SECTION("Transact(Transact(Assign, Transact(Open(Assign))))")
    {
        AutoRTFM::Transact([&]
        {
            AutoRTFM::Transact([&]
            {
                I = 42;
                AutoRTFM::Transact([&]
                {
                    AutoRTFM::Open([&]
                    {
                        SafeToAssignTo = 10;

                        WRITE_THAT_CAUSES_COLLISION(I = 24);

                        SafeToAssignTo = 20;
                    });
                });
            });
        });
    }

    SECTION("Transact(Transact(Assign, Open(Assign, Transact())))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;

                WRITE_THAT_CAUSES_COLLISION(I = 24);

                SafeToAssignTo = 20;

                AutoRTFM::Transact([&] {});
            });
        });
    }

    SECTION("Transact(Transact(Assign, Open(Transact(), Assign)))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open([&]
            {
                AutoRTFM::Transact([&] {});

                SafeToAssignTo = 10;

                WRITE_THAT_CAUSES_COLLISION(I = 24);

                SafeToAssignTo = 20;
            });
        });
    }

    SECTION("Transact(Assign, CallOpen(Assign))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AssignIntPointer(&I, 24, &LineThatPerformsConflictingWrite);
        });
    }

    SECTION("Transact(Assign, Transact(CallOpen(Assign)))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Transact([&]
            {
                AssignIntPointer(&I, 24, &LineThatPerformsConflictingWrite);
            });
        });
    }

    SECTION("Transact(Assign, OpenNoSanitize(Assign))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open<AutoRTFM::EOpenFlags::NoSanitize>([&]
            {
                I = 10;
            });
        });

        REQUIRE(WarningContext.GetWarnings().empty());
    }

    SECTION("Transact(Assign, AlwaysOpenNoSanitize(Assign))")
    {
        struct S
        {
            UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
            static void AssignToInt(int& I)
            {
                I = 10;
            }
        };

        AutoRTFM::Transact([&]
        {
            I = 42;
            S::AssignToInt(I);
        });

        REQUIRE(WarningContext.GetWarnings().empty());
    }

    SECTION("Transact(Assign, Open(Assign, OpenNoSanitize()))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;

                WRITE_THAT_CAUSES_COLLISION(I = 24);

                SafeToAssignTo = 20;

                AutoRTFM::Open<AutoRTFM::EOpenFlags::NoSanitize>([&]{});
            });
        });
    }

    SECTION("Transact(Assign, Open(Assign, Close(OpenNoSanitize())))")
    {
        AutoRTFM::Transact([&]
        {
            I = 42;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;

                WRITE_THAT_CAUSES_COLLISION(I = 24);

                SafeToAssignTo = 20;

                std::ignore = AutoRTFM::Close([&]
                {
                    AutoRTFM::Open<AutoRTFM::EOpenFlags::NoSanitize>([&]{});
                });
            });
        });
    }

    SECTION("Multiple closed writes, single spanning open write")
    {
        // Write to Array[0] and Array[2] transactionally (separate intervals),
        // then do an open write spanning the whole array.
        int Array[4] = {};
        AutoRTFM::Transact([&]
        {
            Array[0] = 1;
            Array[2] = 2;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;
                WRITE_THAT_CAUSES_COLLISION(memset(Array, 0, sizeof(Array)));
                SafeToAssignTo = 20;
            });
        });

        REQUIRE(WarningContext.HasWarningSubstring("conflicting transactional write(s)"));
    }

    SECTION("Partial overlap, open write starts before closed write")
    {
        // Write to Array[1..2] transactionally, then do an open write to Array[0..1]
        // which starts before the closed interval (the SOL-8992 scenario).
        int Array[4] = {};
        AutoRTFM::Transact([&]
        {
            Array[1] = 42;
            AutoRTFM::Open([&]
            {
                SafeToAssignTo = 10;
                WRITE_THAT_CAUSES_COLLISION(memset(&Array[0], 0, 2 * sizeof(int))); // write to Array[0..1]
                SafeToAssignTo = 20;
            });
        });
    }

    if (LineThatPerformsConflictingWrite > 0)
    {
        REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
#if UE_BUILD_DEBUG // Inlining can cause line numbers to differ.
        std::string ExpectedFileAndLine = std::string(__FILE__ ":") + std::to_string(LineThatPerformsConflictingWrite);
        // Trim away directory part of ExpectedFileAndLine, as the slash format
        // may differ by OS delimiter / backtrace library.
        if (size_t LastSlash = ExpectedFileAndLine.find_last_of("\\/"); LastSlash != std::string::npos)
        {
            ExpectedFileAndLine = ExpectedFileAndLine.substr(LastSlash + 1);
        }
        REQUIRE(WarningContext.HasWarningSubstring(ExpectedFileAndLine));
#endif // UE_BUILD_DEBUG
    }
    else
    {
        REQUIRE(WarningContext.GetWarnings().empty());
    }
}
#endif // AUTORTFM_SANITIZER
