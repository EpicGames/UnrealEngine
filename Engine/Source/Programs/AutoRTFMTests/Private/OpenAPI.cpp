// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"

#include <atomic>
#include <map>
#include <thread>
#include <vector>

#if AUTORTFM_SANITIZER
TEST_CASE("OpenAPI.StartAbortAndStartAgain")
{
    AUTORTFM_SCOPED_SANITIZER_MODE_WARN();
    AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	int ValueB = 0;
	int ValueC = 0;
	AutoRTFM::Transact([&]()
	{
		// Recorded ValueB as starting at 0.
		ValueB = 20;

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::RecordOpenWrite(&ValueB);
			ValueB = 10;
			AutoRTFM::AbortTransaction();

			AutoRTFM::StartTransaction();
			AutoRTFM::RecordOpenWrite(&ValueC);
			ValueC = 30;
			AutoRTFM::AbortTransaction();
		});
	});

	// We rollback the transaction to the value we had when we first recorded the address
	REQUIRE(ValueB == 20);
	REQUIRE(ValueC == 0);

	REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
}
#endif // AUTORTFM_SANITIZER

TEST_CASE("OpenAPI.CommitScopedFromOpen")
{
	AutoRTFM::ETransactionResult TransactResult = AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::CommitTransaction();
		});
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::Committed);
}

TEST_CASE("OpenAPI.RecordDataClosed_Illegal", "[.]")
{
	int Value = 0;
	AutoRTFM::ETransactionResult TransactResult = AutoRTFM::Transact([&]()
	{
		AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
		{
			AutoRTFM::RecordOpenWrite(&Value); // Illegal. Can't record writes explicitly while closed
			Value = 1;
		});
	});

	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::Committed);
	REQUIRE(Value == 1);
}

TEST_CASE("OpenAPI.WriteDataInTheOpen")
{
	int Value = 0;
	auto TransactResult = AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::Assign(Value, 1);
		});
	});

	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::Committed);
	REQUIRE(Value == 1);
}

TEST_CASE("OpenAPI.AbortTransaction.Lambda")
{
	auto TransactResult = AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::AbortTransaction();
		});
		FAIL("AutoRTFM::Open failed to throw after an abort");
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
}

TEST_CASE("OpenAPI.AbortTransaction.AUTORTFM_OPEN")
{
	struct S
	{
		AUTORTFM_OPEN static void AbortTransaction()
		{
			AutoRTFM::AbortTransaction();
		}
	};

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		S::AbortTransaction();
		FAIL("AUTORTFM_OPEN failed to throw after an abort");
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
}

TEST_CASE("OpenAPI.AbortTransaction.AUTORTFM_OPEN_NO_VALIDATION")
{
	struct S
	{
		AUTORTFM_OPEN_NO_VALIDATION static void AbortTransaction()
		{
			AutoRTFM::AbortTransaction();
		}
	};

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		S::AbortTransaction();
		FAIL("AUTORTFM_OPEN_NO_VALIDATION failed to throw after an abort");
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
}

TEST_CASE("OpenAPI.AbortTransaction.UE_AUTORTFM_ALWAYS_OPEN")
{
	struct S
	{
		UE_AUTORTFM_ALWAYS_OPEN static void AbortTransaction()
		{
			AutoRTFM::AbortTransaction();
		}
	};

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		S::AbortTransaction();
		FAIL("UE_AUTORTFM_ALWAYS_OPEN failed to throw after an abort");
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
}

TEST_CASE("OpenAPI.AbortTransaction.UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION")
{
	struct S
	{
		UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION static void AbortTransaction()
		{
			AutoRTFM::AbortTransaction();
		}
	};

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		S::AbortTransaction();
		FAIL("UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION failed to throw after an abort");
	});
	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
}

TEST_CASE("OpenAPI.AbortTransactionDoubleScopedFromOpen")
{
	unsigned long Value = -42;
	bool bContinuedExecutionAfterRollback = false;
	AutoRTFM::Testing::Commit([&]()
	{
		Value = 42;

		AutoRTFM::Testing::Abort([&]
		{
			Value = 42424242;
			AutoRTFM::Open([&]
			{
				AutoRTFM::AbortTransaction();
				bContinuedExecutionAfterRollback = true;
			});
			FAIL("AutoRTFM::Open failed to throw after a rollback");
			Value = 24242424;
		});

		if (Value != 42) FAIL("Value != 42");
		Value = 123123123;
	});
	REQUIRE(Value == 123123123);
	REQUIRE(bContinuedExecutionAfterRollback);
}

TEST_CASE("OpenAPI.NestedClosedTransactions")
{
	int Value = 0x12345678;
	auto TransactResult = AutoRTFM::Transact([&]()
	{
		// Read value
		int x = Value;
		Value = 0x11111111;

		auto transactResult2 = AutoRTFM::Transact([&]()
		{
			if (Value != 0x11111111) FAIL("Value != 0x11111111");
			// Read value
			int y = Value;
			Value = 0x22222222;

			auto transactResult3 = AutoRTFM::Transact([&]()
			{
				if (Value != 0x22222222) FAIL("Value != 0x22222222");
				// Read value
				int z = Value;
				Value = 0x33333333;

				auto transactResult4 = AutoRTFM::Transact([&]()
				{
					if (Value != 0x33333333) FAIL("Value != 0x33333333");
					// Read value
					int q = Value;
					Value = 0x44444444;
					if (Value != 0x44444444) FAIL("Value != 0x44444444");
					if (q != 0x33333333) FAIL("q != 0x33333333");
				});
				(void)transactResult4;

				if (Value != 0x44444444) FAIL("Value != 0x44444444");
				if (z != 0x22222222) FAIL("z != 0x22222222");
			});
			(void)transactResult3;

			Value = 0x55555555;
			if (Value != 0x55555555) FAIL("Value != 0x55555555");
			if (y != 0x11111111) FAIL("y != 0x11111111");
		});
		(void)transactResult2;
		if (Value != 0x55555555) FAIL("Value != 0x55555555");

		Value = 0x66666666;
		if (Value != 0x66666666) FAIL("Value != 0x66666666");
		if (x != 0x12345678) FAIL("x != 0x12345678");
	});

	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::Committed);
	REQUIRE(Value == 0x66666666);
}

TEST_CASE("OpenAPI.OpenWithCopy")
{
	struct SomeData_t
	{
		int A;
		float B;
		char C;
	};

	SomeData_t SomeData1{ 1,2.0,'3' };

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		SomeData_t SomeData2{ 9,8.0,'7' };
		SomeData1.A = 11;
		SomeData2.A = 29;

		AutoRTFM::Open([=]()
		{
			REQUIRE(SomeData1.A == 11);
			REQUIRE(SomeData2.A == 29);
		});
	});

	REQUIRE(TransactResult == AutoRTFM::ETransactionResult::Committed);
}

#if defined(_BROKEN_ALLOC_FIXED_)

TEST_CASE("OpenAPI.OpenCloseOpenClose")
{
	// START OPEN
	REQUIRE(!AutoRTFM::IsTransactional());

	int x = 42;
	std::vector<int> v;
	std::map<int, std::vector<int>> m;
	v.push_back(100);
	m[1].push_back(2);
	m[1].push_back(3);
	m[4].push_back(5);
	m[6].push_back(7);
	m[6].push_back(8);
	m[6].push_back(9);

	auto TransactResult = AutoRTFM::Transact([&]()
	{
		// A - WE ARE CLOSED 
		if (!AutoRTFM::IsClosed()) FAIL("A - NOT CLOSED AS EXPECTED!");

		// -------------------------------------
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			// B - WE ARE OPEN 
			REQUIRE(!AutoRTFM::IsClosed());

			// -------------------------------------
			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				// C - WE ARE CLOSED AGAIN
				if (!AutoRTFM::IsClosed()) FAIL("C - NOT CLOSED AS EXPECTED!");

				// -------------------------------------
				AutoRTFM::Open([&] AUTORTFM_DISABLE
				{
					// D - WE ARE OPEN AGAIN
					REQUIRE(!AutoRTFM::IsClosed());

					// An abort here will set state on the transactions, but will not LongJump
					// AutoRTFM::Abort();
				});

				// -------------------------------------
				// E - BACK TO CLOSED AFTER AN OPEN

				x = 5;
				for (size_t n = 10; n--;)
					v.push_back(2 * n);
				m.clear();
				m[10].push_back(11);
				m[12].push_back(13);
				m[12].push_back(14);

				// An abort here is closed and will LongJump past F AND G all the way to H
				AutoRTFM::AbortTransaction();

				// -------------------------------------
				AutoRTFM::Open([&] AUTORTFM_DISABLE
				{
					// F - WE ARE OPEN AGAIN //
					REQUIRE(!AutoRTFM::IsClosed());
				});

				// -------------------------------------
				// G - BACK TO CLOSED AGAIN
				if (!AutoRTFM::IsClosed()) FAIL("NOT CLOSED!");

			});

			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);

			// -------------------------------------
			// H - BACK TO OPEN 
			REQUIRE(!AutoRTFM::IsClosed());

		});

		// -------------------------------------
		// I - Finally closed again to finish out the transaction
		if (!AutoRTFM::IsClosed()) FAIL("I - NOT CLOSED AS EXPECTED!");
	});

	REQUIRE(
		AutoRTFM::ETransactionResult::AbortedByRequest ==
		TransactResult);
	REQUIRE(x == 42);
	REQUIRE(v.size() == 1);
	REQUIRE(v[0] == 100);
	REQUIRE(m.size() == 3);
	REQUIRE(m[1].size() == 2);
	REQUIRE(m[1][0] == 2);
	REQUIRE(m[1][1] == 3);
	REQUIRE(m[4].size() == 1);
	REQUIRE(m[4][0] == 5);
	REQUIRE(m[6].size() == 3);
	REQUIRE(m[6][0] == 7);
	REQUIRE(m[6][1] == 8);
	REQUIRE(m[6][2] == 9);
	REQUIRE(!AutoRTFM::IsTransactional());
}

#endif

TEST_CASE("OpenAPI.Commit_TransactOpenCloseCommit")
{
	REQUIRE(!AutoRTFM::IsTransactional());

	// We're open
	REQUIRE(!AutoRTFM::IsClosed());

	int Value = 10;
	Value++;

	// Close and start the top-level transaction
	AutoRTFM::Transact([&]()
	{
		if (!AutoRTFM::IsClosed()) FAIL("Not Closed");

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				Value = 42;
			});
			
			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);

			REQUIRE(Value == 42); // RTFM writes through immediately, so we can see this value in the open
			AutoRTFM::CommitTransaction();
		});

		if (Value != 42) FAIL("Value != 42!");

		Value = 420;
	});

	REQUIRE(Value == 420);

	REQUIRE(!AutoRTFM::IsTransactional());
}

TEST_CASE("OpenAPI.Commit_TransactOpenCloseRollback")
{
	REQUIRE(!AutoRTFM::IsTransactional());

	// We're open
	REQUIRE(!AutoRTFM::IsClosed());

	int Value = 10;
	Value++;

	bool bExecutedAfterOpen = false;
	
	// Close and start the top-level transaction
	AutoRTFM::Testing::Commit([&]()
	{
		if (!AutoRTFM::IsClosed()) FAIL("Not Closed");

		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				int Local = 0;
				Local = 42;
				Value = Local;
			});

			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);

			AutoRTFM::AbortTransaction(); // undoes Value = 42 in the open
		});

		bExecutedAfterOpen = true;
	});

	REQUIRE(Value == 11);
	REQUIRE(!AutoRTFM::IsTransactional());
	REQUIRE(bExecutedAfterOpen);
}

TEST_CASE("OpenAPI.DoubleTransact")
{
	double Value = 1.0;

	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Transact([&]()
		{
			Value *= 2.5;
			AutoRTFM::AbortTransaction();
		});

		Value *= 10.0;
	});

	REQUIRE(Value == 10.0);
}

TEST_CASE("OpenAPI.DoubleTransact2")
{
	double Value = 1.0;

	AutoRTFM::Transact([&]()
	{
		Value = Value + 2.0;
		AutoRTFM::Transact([&]()
		{
			if (Value == 3.0)
			{
				Value *= 2.5;
			}

			if (Value == 7.5)
			AutoRTFM::AbortTransaction();
		});

		Value *= 10.0;
	});

	REQUIRE(Value == 30.0);
}

TEST_CASE("OpenAPI.DoubleTransact3")
{
	double result = 0.0;
	AutoRTFM::Transact([&]()
	{
		double Value = 1.0;
		Value = Value + 2.0;
		AutoRTFM::Transact([&]()
		{
			if (Value == 3.0)
			{
				Value *= 2.5;
			}

			if (Value == 7.5)
			AutoRTFM::AbortTransaction();
		});

		Value *= 10.0;
		result = Value;
	});

	REQUIRE(result == 30.0);
}

TEST_CASE("OpenAPI.StackWriteCommitInTheOpen1")
{
	int Value = 0;
	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::Assign(Value, 10);
			AutoRTFM::CommitTransaction();
			REQUIRE(Value == 10);
		});
	});
}

TEST_CASE("OpenAPI.StackWriteCommitInTheOpen2")
{
	int Value = 0;

	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::Open([&] AUTORTFM_DISABLE
				{
					AutoRTFM::Assign(Value, 10);
				});
			});

			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);

			AutoRTFM::CommitTransaction();
		});

		REQUIRE(Value == 10);
	});
}

TEST_CASE("OpenAPI.Open(StartTransaction, StackWrite, AbortTransaction)")
{
	int Value = 0;
	bool bRanPostOpen = false;
	AutoRTFM::Testing::Commit([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::Assign(Value, 10);
			AutoRTFM::AbortTransaction();
			REQUIRE(Value == 0);
		});
		bRanPostOpen = true;
	});
	REQUIRE(Value == 0);
	REQUIRE(bRanPostOpen);
}

TEST_CASE("OpenAPI.Open(StackWrite, AbortTransaction)")
{
	int Value = 0;
	bool bRanPostOpen = false;
	AutoRTFM::Testing::Abort([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::Assign(Value, 10);
			AutoRTFM::AbortTransaction();
			REQUIRE(Value == 0);
		});
		bRanPostOpen = true;
	});
	REQUIRE(Value == 0);
	REQUIRE(!bRanPostOpen);
}

#if defined(OPENAPI_ILLEGAL_TESTS)
TEST_CASE("OpenAPI.StackWriteCommitInTheOpen3_Illegal")
{
	AutoRTFM::Transact([&]()
	{
		int Value = 0;
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
			AutoRTFM::RecordOpenWrite(&Value);
			Value = 10;
			AutoRTFM::CommitTransaction();
			REQUIRE(Value == 10);
		});
	});
}
#endif

int Value1 = 0;

TEST_CASE("OpenAPI.WriteMemory1")
{
	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
			
			AutoRTFM::Assign(Value1, 10);

			AutoRTFM::CommitTransaction();
		});
		REQUIRE(Value1 == 10);
	});
}

TEST_CASE("OpenAPI.WriteTrivialStructure")
{
	struct SomeData
	{
		int A;
		double B;
		float C;
		char D;
		long E[5];
	};

	SomeData Data = { 1,2.0, 3.0f, 'q', {123,234,345,456,567} };
	SomeData Data2 = { 9,8.0, 7.0f, '^', {999,888,777,666,555} };

	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			AutoRTFM::Assign(Data, Data2);
			REQUIRE(Data.A == 9);
			REQUIRE(Data.B == 8.0);
			REQUIRE(Data.C == 7.0f);
			REQUIRE(Data.D == '^');
			REQUIRE(Data.E[0] == 999);
			REQUIRE(Data.E[1] == 888);
			REQUIRE(Data.E[2] == 777);
			REQUIRE(Data.E[3] == 666);
			REQUIRE(Data.E[4] == 555);
			
			AutoRTFM::AbortTransaction();
			REQUIRE(Data.A == 1);
			REQUIRE(Data.B == 2.0);
			REQUIRE(Data.C == 3.0f);
			REQUIRE(Data.D == 'q');
			REQUIRE(Data.E[0] == 123);
			REQUIRE(Data.E[1] == 234);
			REQUIRE(Data.E[2] == 345);
			REQUIRE(Data.E[3] == 456);
			REQUIRE(Data.E[4] == 567);
		});
	});
}

TEST_CASE("OpenAPI.WriteTrivialStructure2")
{
	struct SomeData
	{
		int A;
		double B;
		float C;
		char D;
		long E[5];
	};

	SomeData Data = { 1,2.0, 3.0f, 'q', {123,234,345,456,567} };
	SomeData Data2 = { 9,8.0, 7.0f, '^', {999,888,777,666,555} };
	SomeData Data3 = { 19,28.0, 37.0f, '@', {4999,5888,6777,7666,8555} };

	AutoRTFM::Transact([&]()
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();
				
			AutoRTFM::Assign(Data, Data2);
			REQUIRE(Data.A == 9);
			REQUIRE(Data.B == 8.0);
			REQUIRE(Data.C == 7.0f);
			REQUIRE(Data.D == '^');
			REQUIRE(Data.E[0] == 999);
			REQUIRE(Data.E[1] == 888);
			REQUIRE(Data.E[2] == 777);
			REQUIRE(Data.E[3] == 666);
			REQUIRE(Data.E[4] == 555);

			AutoRTFM::Assign(Data, Data3);
			REQUIRE(Data.A == 19);
			REQUIRE(Data.B == 28.0);
			REQUIRE(Data.C == 37.0f);
			REQUIRE(Data.D == '@');
			REQUIRE(Data.E[0] == 4999);
			REQUIRE(Data.E[1] == 5888);
			REQUIRE(Data.E[2] == 6777);
			REQUIRE(Data.E[3] == 7666);
			REQUIRE(Data.E[4] == 8555);

			AutoRTFM::AbortTransaction();
			REQUIRE(Data.A == 1);
			REQUIRE(Data.B == 2.0);
			REQUIRE(Data.C == 3.0f);
			REQUIRE(Data.D == 'q');
			REQUIRE(Data.E[0] == 123);
			REQUIRE(Data.E[1] == 234);
			REQUIRE(Data.E[2] == 345);
			REQUIRE(Data.E[3] == 456);
			REQUIRE(Data.E[4] == 567);
		});
	});
}

#if AUTORTFM_SANITIZER
TEST_CASE("OpenAPI.Footgun1")
{
	AUTORTFM_SCOPED_SANITIZER_MODE_WARN();
	AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	int ValueA = 0;
	int ValueB = 0;

	AutoRTFM::Transact([&]()
	{
		// Does nothing - already closed
		AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
		{
			// Recorded ValueB as starting at 0.
			ValueB = 123;

			AutoRTFM::Open([&] AUTORTFM_DISABLE
			{
				// Unrecorded assignments in the open
				ValueA = 10;
				ValueB = 10;
			});
				
			// ValueA is now recorded as starting at 10
			ValueA = 20;
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(AutoRTFM::ETransactionStatus::AbortedByRequest == Status);
	});

	// We rollback the transaction to the value we had when we first recorded the address
	REQUIRE(ValueA == 10);
	REQUIRE(ValueB == 0);

	REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
}
#endif // AUTORTFM_SANITIZER

#if AUTORTFM_SANITIZER
TEST_CASE("OpenAPI.Footgun2")
{
	AUTORTFM_SCOPED_SANITIZER_MODE_WARN();
	AutoRTFMTestUtils::FCaptureWarningContext WarningContext;

	int ValueB = 0;
	int ValueC = 0;
	AutoRTFM::Transact([&]()
	{
		// Does nothing - already closed
		AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
		{
			// Recorded ValueB as starting at 0.
			ValueB = 20;

			AutoRTFM::Open([&] AUTORTFM_DISABLE
			{
				// Unrecorded assignments in the open
				ValueB = 10;
				ValueC = 10;
				AutoRTFM::RecordOpenWrite(&ValueC);
				// ValueC was recorded in the open after the change - too late
			});

			// ValueA is now recorded as starting at 10
			ValueC = 40;
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(AutoRTFM::ETransactionStatus::AbortedByRequest == Status);
	});

	// We rollback the transaction to the value we had when we first recorded the address
	REQUIRE(ValueB == 0);
	REQUIRE(ValueC == 10);

	REQUIRE(WarningContext.HasWarningSubstring(AutoRTFMTestUtils::kMemoryModifiedInOpenWarning));
}
#endif // AUTORTFM_SANITIZER

#if 0
TEST_CASE("OpenAPI.StartCloseOnCommit")
{
	REQUIRE(!AutoRTFM::IsTransactional());

	int Value = 10;
	Value++;
	(void)Value;

	AutoRTFM::StartTransaction();

	// Can't close outside of a transaction
	AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
	{
		Value = 420;
	});
	
	REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);

	// assignment within the close should be visible to us
	REQUIRE(Value == 420);

	// Setting a value in the open requires us to register the memory address with the transaction
	Value = 42;
	AutoRTFM::RecordOpenWrite(&Value);

	AutoRTFM::CommitTransaction();

	// Finally, 42 is committed to Value
	REQUIRE(Value == 42);

	REQUIRE(!AutoRTFM::IsTransactional());
}
#endif

TEST_CASE("OpenAPI.TransOpenStartCloseAbortAbort")
{
	bool bGetsToA = false;
	bool bGetsToB = false;
	bool bGetsToC = false;
	bool bGetsToD = false;

	REQUIRE(!AutoRTFM::IsTransactional());

	int Value = 10;

	AutoRTFM::Transact([&]() 
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::StartTransaction();

			Value++;

			Value = 42;
			AutoRTFM::ETransactionStatus CloseStatus = AutoRTFM::Close([&]
			{
				Value = 420;
				AutoRTFM::AbortTransaction();
				bGetsToA = true;
			});

			REQUIRE(CloseStatus == AutoRTFM::ETransactionStatus::AbortedByRequest);

			REQUIRE(bGetsToA == false);

			bGetsToB = true;
			REQUIRE(Value == 42);
			AutoRTFM::AbortTransaction();
			bGetsToC = true;
			REQUIRE(Value == 42);
		});

		bGetsToD = true;
	});

	REQUIRE(bGetsToA == false);
	REQUIRE(bGetsToB == true);
	REQUIRE(bGetsToC == true);
	REQUIRE(bGetsToD == false);
	REQUIRE(!AutoRTFM::IsTransactional());
}

TEST_CASE("OpenAPI.TransOpenTransCloseAbortAbort")
{
	bool bGetsToA = false;
	bool bGetsToB = false;
	bool bGetsToC = false;
	bool bGetsToD = false;

	REQUIRE(!AutoRTFM::IsTransactional());

	AutoRTFM::Transact([&]() 
	{
		AutoRTFM::Open([&] AUTORTFM_DISABLE
		{
			AutoRTFM::Transact([&]()
			{
				int Value = 10;
				Value++;

				Value = 42;

				AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
				{
					Value = 420;
					AutoRTFM::AbortTransaction();
					bGetsToA = true;
				});

				REQUIRE(AutoRTFM::ETransactionStatus::AbortedByRequest == Status);

				REQUIRE(bGetsToA == false);

				bGetsToB = true;
				REQUIRE(Value == 42);
				AutoRTFM::AbortTransaction();
				bGetsToC = true;
				REQUIRE(Value == 42);
			});
		});

		bGetsToD = true;
	});

	REQUIRE(bGetsToA == false);
	REQUIRE(bGetsToB == false);
	REQUIRE(bGetsToC == false);
	REQUIRE(bGetsToD == true);
	REQUIRE(!AutoRTFM::IsTransactional());
}

TEST_CASE("OpenAPI.DeferredStartTransactionOverflow")
{
	AutoRTFM::TransactThenOpen([&]
	{
		uint64 NestingCount = static_cast<uint64>(std::numeric_limits<uint16>::max()) + 42;
		for (uint64 I = 0; I < NestingCount; ++I)
		{
			AutoRTFM::StartTransaction();
		}
		for (uint64 I = 0; I < NestingCount; ++I)
		{
			if (!!(I % 2))
			{
				AutoRTFM::CommitTransaction();
			}
			else
			{
				AutoRTFM::AbortTransaction();
			}
		}
	});
}

TEST_CASE("OpenAPI.CheckRaceAgainstOtherThread")
{
	bool bHit = false;
	std::atomic_uint Handshake = 0;
	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
	{
		AutoRTFM::Open([&]
		{
			std::thread([&] AUTORTFM_ENABLE // SOL-8741
			{
				// This should be a no-op in the spawnee thread.
				AutoRTFM::RecordOpenWrite(&bHit);
				bHit = true;

				// Unblock the main thread.
				Handshake += 1;
			}).detach();

			// Wait for the spawnee.
			while (1 != Handshake) {}
		});

		AutoRTFM::AbortTransaction();
	});

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	REQUIRE(bHit);
}
