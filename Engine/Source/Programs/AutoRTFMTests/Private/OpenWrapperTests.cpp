// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/OpenWrapper.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

TEST_CASE("OpenWrapper")
{
	struct FObject
	{
		FObject()
		{
			REQUIRE(!AutoRTFM::IsClosed());
		}
		FObject(const FObject&)
		{
			REQUIRE(!AutoRTFM::IsClosed());
		}
		FObject(FObject&&)
		{
			REQUIRE(!AutoRTFM::IsClosed());
		}
		FObject& operator=(const FObject&)
		{
			REQUIRE(!AutoRTFM::IsClosed());
			return *this;
		}
		FObject& operator=(FObject&&)
		{
			REQUIRE(!AutoRTFM::IsClosed());
			return *this;
		}
		~FObject()
		{
			REQUIRE(!AutoRTFM::IsClosed());
		}
	};

	FObject Object;
	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::TOpenWrapper WrapperA{Object};
		AutoRTFM::TOpenWrapper WrapperB{std::move(WrapperA)};
		AutoRTFM::TOpenWrapper WrapperC = WrapperB;
		AutoRTFM::TOpenWrapper WrapperD = std::move(WrapperC);
	});
}
