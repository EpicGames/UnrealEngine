// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

#include <inttypes.h>

#pragma pack(push,1)
struct Foo
{
    uint8_t a;
    uint64_t b;
};
#pragma pack(pop)

TEST_CASE("PackedPointer")
{
    Foo foo;
    foo.a = 1;
    foo.b = 2;
    AutoRTFM::Testing::Commit([&] ()
    {
        foo.a++;
        foo.b++;
    });
    REQUIRE(foo.a == 2);
    REQUIRE(foo.b == 3);
}

