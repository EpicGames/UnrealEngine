// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "Pool.h"
#include "Catch2Includes.h"
#include "ObjectLifetimeHelper.h"

#include <algorithm>
#include <vector>

using TrivialPool4 = AutoRTFM::TPool<int, 4>;
using TrivialPool64 = AutoRTFM::TPool<int, 64>;
using NonTrivialPool4 = AutoRTFM::TPool<AutoRTFMTestUtils::FObjectLifetimeHelper, 4>;
using NonTrivialPool64 = AutoRTFM::TPool<AutoRTFMTestUtils::FObjectLifetimeHelper, 64>;

TEMPLATE_TEST_CASE("Pool", "",  TrivialPool4, TrivialPool64, NonTrivialPool4, NonTrivialPool64)
{
	AutoRTFM::UnreachableIfClosed();

	Catch::SimplePcg32 Rand(0x1234);
	TestType Pool;
	using TItem = typename TestType::FItem;


	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == 0);
	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls == 0);

	for (int Count : {1, 2, 3, 4, 5, 8, 10, 100})
	{
		// Take Count items from the pool
		std::vector<TItem*> Items(Count);
		for (int I = 0; I < Count; I++)
		{
			TItem* Item = Pool.Take(I);
			REQUIRE(*Item == I);
			Items[I] = Item;
		}

		// Check no changes to the items as new items are taken.
		for (int I = 0; I < Count; I++)
		{
			REQUIRE(*Items[I] == I);
		}

		// Shuffle the order of the items
		std::shuffle(Items.begin(), Items.end(), Rand);

		// Return the items, but keep track of the pointers.
		for (TItem* Item : Items)
		{
			Pool.Return(Item);
		}

		// Retake the items and check for LIFO ordering.
		for (int I = 0; I < Count; I++)
		{
			TItem* Item = Pool.Take(1000 + I);
			REQUIRE(*Item == 1000 + I);
			REQUIRE(Items[Count - I - 1] == Item); // Check for LIFO
		}

		// Check no changes to the items as new items are taken.
		for (int I = 0; I < Count; I++)
		{
			REQUIRE(*Items[Count - I - 1] == 1000 + I);
		}

		// Return all the items.
		for (TItem* Item : Items)
		{
			Pool.Return(Item);
		}
	}
	
	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls);
	AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls = 0;
	AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls = 0;
}
