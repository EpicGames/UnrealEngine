// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "Containers/TransactionallySafeSpscQueue.h"
#include "Containers/TransactionallySafeMpscQueue.h"

#include <thread>

// Test loosely based on code from Containers/ConcurrentQueuesTest.cpp.
template <typename QueueType, unsigned NumProducerThreads = 1>
void TestQueueCorrectness(int Num, bool bTransact, typename QueueType::ElementType (*Produce)(int Index))
{
	QueueType Queue;
	std::thread ConsumeTask;
	std::vector<std::thread> ProduceTasks;
	std::atomic<bool> BusyWait = true;
	std::set<typename QueueType::ElementType> ElementsConsumed;

	// Consumer
	auto ConsumeLoop = [&Queue, Num, &ElementsConsumed]
	{
		while (ElementsConsumed.size() != (Num * NumProducerThreads))
		{
			TOptional<typename QueueType::ElementType> Element = Queue.Dequeue();

			if (Element.IsSet())
			{
				ElementsConsumed.insert(Element.GetValue());
			}
		}
	};

	// Producer
	auto ProduceLoop = [&Queue, Num, &BusyWait, Produce]
	{
		for (int Index = 0; Index < Num; Index++)
		{
			Queue.Enqueue(Produce(Index));

			if (Index == 0)
			{
				// We signal that other threads can start producing
				// after we've produced one element.
				UE_AUTORTFM_OPEN
				{
					BusyWait = false;
					BusyWait.notify_all();
				};
			}
		}
	};

	// Threaded producer
	auto ThreadedProduceLoop = [&Queue, Num, &BusyWait, Produce](unsigned ThreadId)
	{
		BusyWait.wait(true);

		for (int Index = 0; Index < Num; Index++)
		{
			Queue.Enqueue(Produce(ThreadId * Num + Index));
		}
	};
	
	// Run the consumer loop on a helper thread.
	ConsumeTask = std::thread{ConsumeLoop};

	// We start at 1 because our current thread will produce items too.
	for (unsigned Thread = 1; Thread < NumProducerThreads; Thread++)
	{
		ProduceTasks.push_back(std::thread(ThreadedProduceLoop, Thread));
	}
	
	// Run the producer loop within a transaction.
	if (bTransact)
	{
		AutoRTFM::Testing::Commit([&]
		{
			ProduceLoop();
		});
	}
	else
	{
		ProduceLoop();
	}

	// Join the consumer thread to allow it to finish.
	ConsumeTask.join();

	for (std::thread& ProduceTask : ProduceTasks)
	{
		ProduceTask.join();
	}

	// We should have produced and consumed exactly Num items, and the queue should be empty.
	REQUIRE(ElementsConsumed.size() == (Num * NumProducerThreads));
	REQUIRE(!Queue.Dequeue().IsSet());

	for (int Index = 0; Index < (Num * NumProducerThreads); Index++)
	{
		REQUIRE(ElementsConsumed.contains(Produce(Index)));
	}
}

template<template<typename> class QueueType> struct TQueueWithInt final
{
	using Queue = QueueType<int>;

	static int Producer(int Index);
};

template<template<typename> class QueueType> int TQueueWithInt<QueueType>::Producer(int Index)
{
	return Index;
}

template<template<typename> class QueueType> struct TQueueWithString final
{
	using Queue = QueueType<FString>;

	static FString Producer(int Index);
};

template<template<typename> class QueueType> FString TQueueWithString<QueueType>::Producer(int Index)
{
	return FString::Printf(TEXT("%d"), Index);
}

// Test with trivial and complex types
TEMPLATE_TEST_CASE("TransactionallySafeSpscQueue", "",
	TQueueWithInt<TTransactionallySafeSpscQueue>, TQueueWithString<TTransactionallySafeSpscQueue>,
	TQueueWithInt<TTransactionallySafeMpscQueue>, TQueueWithString<TTransactionallySafeMpscQueue>)
{
	// Non-transactional
	TestQueueCorrectness<typename TestType::Queue>(10000, /*bTransact=*/false, TestType::Producer);

	// Transactional
	TestQueueCorrectness<typename TestType::Queue>(10000, /*bTransact=*/true, TestType::Producer);
}

// Test with trivial and complex types
TEMPLATE_TEST_CASE("TransactionallySafeMpscQueue", "",
	TQueueWithInt<TTransactionallySafeMpscQueue>, TQueueWithString<TTransactionallySafeMpscQueue>)
{
	// Non-transactional
	TestQueueCorrectness<typename TestType::Queue, 8>(10000, /*bTransact=*/false, TestType::Producer);

	// Transactional
	TestQueueCorrectness<typename TestType::Queue, 8>(10000, /*bTransact=*/true, TestType::Producer);
}
