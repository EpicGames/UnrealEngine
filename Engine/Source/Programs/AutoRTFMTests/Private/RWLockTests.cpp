// Copyright Epic Games, Inc. All Rights Reserved.

#include "Catch2Includes.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/ScopeRWLock.h"

#include <thread>
#include <mutex>

// Helper class for TEMPLATE_TEST_CASE, which expects a type but we want to pass a value.
template <FRWScopeLockType T>
struct TRWLockTypeHolder
{
	static constexpr FRWScopeLockType Value = T;
};

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.RWScopeLock", "[ScopeLock]", TRWLockTypeHolder<FRWScopeLockType::SLT_ReadOnly>, TRWLockTypeHolder<FRWScopeLockType::SLT_Write>)
{
	SECTION("Outside Transaction With Lock")
	{
		FTransactionallySafeRWLock RWLock;

		AutoRTFM::Testing::Abort([&]
		{
			UE::TRWScopeLock Lock(RWLock, TestType::Value);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			UE::TRWScopeLock Lock(RWLock, TestType::Value);
		});
	}

	SECTION("Inside Transaction With Lock")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			UE::TRWScopeLock Lock(RWLock, TestType::Value);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			UE::TRWScopeLock Lock(RWLock, TestType::Value);
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.NestedRWScopeLock", "[ScopeLock]", TRWLockTypeHolder<FRWScopeLockType::SLT_ReadOnly>, TRWLockTypeHolder<FRWScopeLockType::SLT_Write>)
{
	SECTION("Abort(Abort(Lock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				UE::TRWScopeLock Lock(RWLock, TestType::Value);
				AutoRTFM::CascadingAbortTransaction();
			});
		});
	}

	SECTION("Commit(Abort(Lock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				UE::TRWScopeLock Lock(RWLock, TestType::Value);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Abort(Commit(Lock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				UE::TRWScopeLock Lock(RWLock, TestType::Value);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Commit(Commit(Lock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				UE::TRWScopeLock Lock(RWLock, TestType::Value);
			});
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.ScopeLock", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	SECTION("Outside Transaction")
	{
		FTransactionallySafeRWLock RWLock;

		AutoRTFM::Testing::Abort([&]
		{
			TestType Lock(RWLock);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			TestType Lock(RWLock);
		});
	}

	SECTION("Inside Transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			TestType Lock(RWLock);
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			TestType Lock(RWLock);
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.NestedScopeLock", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	SECTION("Abort(Abort(Lock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				TestType Lock(RWLock);
				AutoRTFM::CascadingAbortTransaction();
			});
		});
	}

	SECTION("Commit(Abort(Lock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				TestType Lock(RWLock);
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Abort(Commit(Lock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				TestType Lock(RWLock);
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Commit(Commit(Lock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				TestType Lock(RWLock);
			});
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.ScopeLock.FailedTryInTransaction", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	FTransactionallySafeRWLock RWLock;
	TestType Lock(RWLock);

	SECTION("TryWrite")
	{
		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(!RWLock.TryWriteLock());
				AutoRTFM::CascadingAbortTransaction();
			});
		}

		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(!RWLock.TryWriteLock());
			});
		}
	}

	// A perfect RW lock would allow multiple TryReads to occur safely, even inside a transaction. For now, we
	// pessimize read locks in the closed, and so the second TryRead is not expected to succeed. #jira SOL-7429
	SECTION("TryRead")
	{
		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(!RWLock.TryReadLock());
				AutoRTFM::CascadingAbortTransaction();
			});
		}

		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(!RWLock.TryReadLock());
			});
		}
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.ZeroAndReconstruct", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	SECTION("Lock, destruct, memzero, reconstruct")
	{
		FTransactionallySafeRWLock RWLock;
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				{
					// Lock and then unlock
					TestType Lock(RWLock);
				}
				RWLock.~FTransactionallySafeRWLock();
				memset(&RWLock, 0, sizeof(RWLock));
				new (&RWLock) FTransactionallySafeRWLock();
			});
		}
		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				{
					// Lock and then unlock
					TestType Lock(RWLock);
				}
				RWLock.~FTransactionallySafeRWLock();
				memset(&RWLock, 0, sizeof(RWLock));
				new (&RWLock) FTransactionallySafeRWLock();
				AutoRTFM::AbortTransaction();
			});
		}
	}
}

TEST_CASE("TransactionallySafeRWLock.TryWriteLock")
{
	SECTION("Declared Outside Transaction")
	{
		FTransactionallySafeRWLock RWLock;

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(RWLock.TryWriteLock());
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(RWLock.TryWriteLock());
			RWLock.WriteUnlock();
		});
	}

	SECTION("Declared Inside Transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryWriteLock());
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryWriteLock());
			RWLock.WriteUnlock();
		});
	}
}

TEST_CASE("TransactionallySafeRWLock.TryReadLock")
{
	SECTION("Declared Outside Transaction")
	{
		FTransactionallySafeRWLock RWLock;

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(RWLock.TryReadLock());
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(RWLock.TryReadLock());
			RWLock.WriteUnlock();
		});
	}

	SECTION("Declared Inside Transaction")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryReadLock());
			AutoRTFM::AbortTransaction();
		});

		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryReadLock());
			RWLock.WriteUnlock();
		});
	}
}

TEST_CASE("TransactionallySafeRWLock.NestedTryWriteLock")
{
	SECTION("Abort(Abort(TryWriteLock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(RWLock.TryWriteLock());
				AutoRTFM::CascadingAbortTransaction();
			});
		});
	}

	SECTION("Commit(Abort(TryWriteLock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(RWLock.TryWriteLock());
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Abort(Commit(TryWriteLock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(RWLock.TryWriteLock());
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Commit(Commit(TryWriteLock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(RWLock.TryWriteLock());
			});

			RWLock.WriteUnlock();
		});
	}
}

TEST_CASE("TransactionallySafeRWLock.NestedTryReadLock")
{
	SECTION("Abort(Abort(TryReadLock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(RWLock.TryReadLock());
				AutoRTFM::CascadingAbortTransaction();
			});
		});
	}

	SECTION("Commit(Abort(TryReadLock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(RWLock.TryReadLock());
				AutoRTFM::AbortTransaction();
			});
		});
	}

	SECTION("Abort(Commit(TryReadLock))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(RWLock.TryReadLock());
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Commit(Commit(TryReadLock))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;

			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(RWLock.TryReadLock());
			});

			RWLock.ReadUnlock();
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.RWScopeLock.ScopeLockThenTryWriteLock", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	SECTION("Lock, Abort(TryWrite)")
	{
		FTransactionallySafeRWLock RWLock;
		TestType Lock(RWLock);

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(!RWLock.TryWriteLock());
			AutoRTFM::CascadingAbortTransaction();
		});
	}

	SECTION("Lock, Commit(TryWrite)")
	{
		FTransactionallySafeRWLock RWLock;
		TestType Lock(RWLock);

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryWriteLock());
		});
	}
}

TEMPLATE_TEST_CASE("TransactionallySafeRWLock.RWScopeLock.ScopeLockThenTryReadLock", "[ScopeLock]", UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TWriteScopeLock<FTransactionallySafeRWLock>)
{
	SECTION("Lock, Abort(TryRead)")
	{
		FTransactionallySafeRWLock RWLock;
		TestType Lock(RWLock);

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(!RWLock.TryReadLock());
			AutoRTFM::CascadingAbortTransaction();
		});
	}

	SECTION("Lock, Commit(TryRead)")
	{
		FTransactionallySafeRWLock RWLock;
		TestType Lock(RWLock);

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryReadLock());
		});
	}
}

TEST_CASE("TransactionallySafeRWLock.MultipleTryLocks")
{
	SECTION("TryWrite, TryWrite, Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryWriteLock());
		REQUIRE(!RWLock.TryWriteLock());
		RWLock.WriteUnlock();
	}

	SECTION("TryWrite, TryRead, Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryWriteLock());
		REQUIRE(!RWLock.TryReadLock());
		RWLock.WriteUnlock();
	}

	SECTION("TryRead, TryRead, Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryReadLock());
		REQUIRE(RWLock.TryReadLock());
		RWLock.ReadUnlock();
		RWLock.ReadUnlock();
	}

	SECTION("TryWrite, Commit(TryWrite), Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryWriteLock());
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryWriteLock());
		});
		RWLock.WriteUnlock();
	}

	SECTION("TryWrite, Commit(TryRead), Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryWriteLock());
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryReadLock());
		});
		RWLock.WriteUnlock();
	}

	SECTION("TryRead, Commit(TryWrite), Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryReadLock());
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryWriteLock());
		});
		RWLock.ReadUnlock();
	}

	// A perfect RW lock would allow multiple TryReads to occur safely, even inside a transaction. For now, we
	// pessimize read locks in the closed, and so the second TryRead is not expected to succeed. #jira SOL-7429
	SECTION("TryRead, Commit(TryRead), Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryReadLock());
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!RWLock.TryReadLock());
		});
		RWLock.ReadUnlock();
	}

	SECTION("TryWrite, Abort(TryWrite), Unlock")
	{
		FTransactionallySafeRWLock RWLock;
		REQUIRE(RWLock.TryWriteLock());
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(!RWLock.TryWriteLock());
			AutoRTFM::AbortTransaction();
		});
		RWLock.WriteUnlock();
	}

	// Test case does not pass. RWLock cannot be locked in the open and unlocked in the closed: #jira SOL-7661
	SECTION("TryWrite, Commit(TryWrite, Unlock)")
	{
//		FTransactionallySafeRWLock RWLock;
//		REQUIRE(RWLock.TryWriteLock());
//		AutoRTFM::Testing::Commit([&]
//		{
//			REQUIRE(!RWLock.TryWriteLock());
//			RWLock.WriteUnlock();
//		}); 
	}

	// Test case does not pass. RWLock cannot be locked in the open and unlocked in the closed: #jira SOL-7661
	SECTION("TryWrite, Abort(TryWrite, Unlock)")
	{
//		FTransactionallySafeRWLock RWLock;
//		REQUIRE(RWLock.TryWriteLock());
//		AutoRTFM::Testing::Abort([&]
//		{
//			REQUIRE(!RWLock.TryWriteLock());
//			RWLock.WriteUnlock();
//			AutoRTFM::AbortTransaction();
//		}); 
//		RWLock.WriteUnlock();
	}


	// Test case does not pass. RWLock cannot be locked in the closed and unlocked in the open: #jira SOL-7661
	SECTION("Commit(TryWrite, TryWrite), Unlock")
	{
//		FTransactionallySafeRWLock RWLock;
//		AutoRTFM::Testing::Commit([&]
//		{
//			REQUIRE(RWLock.TryWriteLock());
//			REQUIRE(!RWLock.TryWriteLock());
//		});
//		RWLock.WriteUnlock();
	}

	SECTION("Abort(TryWrite, TryWrite)")
	{
		FTransactionallySafeRWLock RWLock;
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(RWLock.TryWriteLock());
			REQUIRE(!RWLock.TryWriteLock());
			AutoRTFM::AbortTransaction();
		});
	}

	// A perfect RW lock would allow multiple TryReads to occur safely, even inside a transaction. For now, we
	// pessimize read locks in the closed, and so the second TryRead is not expected to succeed. #jira SOL-7429
	SECTION("Abort(TryRead, TryRead)")
	{
		FTransactionallySafeRWLock RWLock;
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(RWLock.TryReadLock());
			REQUIRE(!RWLock.TryReadLock());
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Commit(TryWrite, TryWrite, Unlock)")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryWriteLock());
			REQUIRE(!RWLock.TryWriteLock());
			RWLock.WriteUnlock();
		});
	}

	// A perfect RW lock would allow multiple TryReads to occur safely, even inside a transaction. For now, we
	// pessimize read locks in the closed, and so the second TryRead is not expected to succeed. #jira SOL-7429
	SECTION("Commit(TryRead, TryRead, Unlock)")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryReadLock());
			REQUIRE(!RWLock.TryReadLock());
			RWLock.ReadUnlock();
		});
	}

	SECTION("Abort(TryWrite, TryWrite, Unlock)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryWriteLock());
			REQUIRE(!RWLock.TryWriteLock());
			RWLock.WriteUnlock();
			AutoRTFM::AbortTransaction();
		});
	}

	// A perfect RW lock would allow multiple TryReads to occur safely, even inside a transaction. For now, we
	// pessimize read locks in the closed, and so the second TryRead is not expected to succeed. #jira SOL-7429
	SECTION("Abort(TryRead, TryRead, Unlock)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTransactionallySafeRWLock RWLock;
			REQUIRE(RWLock.TryReadLock());
			REQUIRE(!RWLock.TryReadLock());
			RWLock.ReadUnlock();
			AutoRTFM::AbortTransaction();
		});
	}
}

TEST_CASE("Benchmarks.TransactionallySafeRWLock")
{
	BENCHMARK("Transactional.ReadLockWriteLock")
	{
		for (int I = 0; I < 100; I++)
		{
			AutoRTFM::Testing::Commit([]
			{
				FTransactionallySafeRWLock RWLock;
				for (int J = 0; J < 100; J++)
				{
					{
						UE::TReadScopeLock ReadLock{RWLock};
					}
					{
						UE::TWriteScopeLock WriteLock{RWLock};
					}
				}
			});
		}
	};
}
