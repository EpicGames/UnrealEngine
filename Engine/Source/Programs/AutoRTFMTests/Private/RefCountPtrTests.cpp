// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "Templates/RefCounting.h"

namespace
{
	struct FRefCountedInt : public FRefCountedObject
	{
		explicit FRefCountedInt(int InValue, bool* InWasDestructed = nullptr) : Value(InValue), WasDestructed(InWasDestructed)
		{
			if (WasDestructed)
			{
				*WasDestructed = false;
			}
		}

		int Value;
		bool* WasDestructed;

		~FRefCountedInt()
		{
			if (WasDestructed)
			{
				*WasDestructed = true;
			}
		}
	};

	struct FRefCountedMixinInt : public TRefCountingMixin<FRefCountedMixinInt>
	{
		explicit FRefCountedMixinInt(int InValue, bool* InWasDestructed = nullptr) : Value(InValue), WasDestructed(InWasDestructed)
		{
			if (WasDestructed)
			{
				*WasDestructed = false;
			}
		}

		int Value;
		bool* WasDestructed;

		~FRefCountedMixinInt()
		{
			if (WasDestructed)
			{
				*WasDestructed = true;
			}
		}
	};

	struct FThreadSafeRefCountedInt : public FRefCountedObject
	{
		explicit FThreadSafeRefCountedInt(int InValue, bool* InWasDestructed = nullptr) : Value(InValue), WasDestructed(InWasDestructed)
		{
			if (WasDestructed)
			{
				*WasDestructed = false;
			}
		}

		int Value;
		bool* WasDestructed;

		~FThreadSafeRefCountedInt()
		{
			if (WasDestructed)
			{
				*WasDestructed = true;
			}
		}
	};

	struct FQueryableRefCountedInt : public UE::Private::FQueryableRefCountedObject
	{
		explicit FQueryableRefCountedInt(int InValue, bool* InWasDestructed = nullptr) : Value(InValue), WasDestructed(InWasDestructed)
		{
			if (WasDestructed)
			{
				*WasDestructed = false;
			}
		}

		int Value;
		bool* WasDestructed;

		~FQueryableRefCountedInt()
		{
			if (WasDestructed)
			{
				*WasDestructed = true;
			}
		}
	};
}

TEMPLATE_TEST_CASE("CheckRefCounts", "[RefCountPtr]", FQueryableRefCountedInt)
{
	// Refcounts should be correct outside of a transaction.
	SECTION("Non-transactional refcounts should be accurate")
	{
		TestType* Ptr = new TestType(42);

		REQUIRE(Ptr->GetRefCount() == 0);
		REQUIRE(Ptr->AddRef() == 1);
		REQUIRE(Ptr->GetRefCount() == 1);
		REQUIRE(Ptr->AddRef() == 2);
		REQUIRE(Ptr->GetRefCount() == 2);
		REQUIRE(Ptr->Release() == 1);
		REQUIRE(Ptr->GetRefCount() == 1);
		REQUIRE(Ptr->AddRef() == 2);
		REQUIRE(Ptr->GetRefCount() == 2);
		REQUIRE(Ptr->Release() == 1);
		REQUIRE(Ptr->GetRefCount() == 1);
		REQUIRE(Ptr->Release() == 0);

		// Ptr is dead
	}

	// Refcounts can be inflated inside a transaction, since releases are deferred.
	SECTION("Transactional refcounts should represent a lower bound")
	{
		AutoRTFM::Testing::Commit([]
		{
			TestType* Ptr = new TestType(42);

			REQUIRE(Ptr->GetRefCount() == 0);
			REQUIRE(Ptr->AddRef() == 1);
			REQUIRE(Ptr->GetRefCount() == 1);
			REQUIRE(Ptr->AddRef() == 2);
			REQUIRE(Ptr->GetRefCount() == 2);
			REQUIRE(Ptr->Release() == 1);
			REQUIRE(Ptr->GetRefCount() == 1);
			REQUIRE(Ptr->AddRef() == 2);
			REQUIRE(Ptr->GetRefCount() == 2);
			REQUIRE(Ptr->Release() == 1);
			REQUIRE(Ptr->GetRefCount() == 1);
			REQUIRE(Ptr->Release() == 0);

			// Ptr is dead
		});
	}
}

TEMPLATE_TEST_CASE("CheckRefCountsToDeprecate", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt)
{
	// Refcounts should be correct outside of a transaction.
	SECTION("Non-transactional refcounts should be accurate")
	{
		bool bWasDestructed = false;

		{
			TestType* Ptr = new TestType(42, &bWasDestructed);

			Ptr->AddRef();
			Ptr->AddRef();
			Ptr->Release();
			Ptr->AddRef();
			Ptr->Release();

			REQUIRE(!bWasDestructed);
			Ptr->Release();

			// We aren't in a transaction so we will have destructed the reference already!
			REQUIRE(bWasDestructed);
		}

		REQUIRE(bWasDestructed);
	}

	// Refcounts can be inflated inside a transaction, since releases are deferred.
	SECTION("Transactional refcounts should represent a lower bound")
	{
		bool bWasDestructed = false;

		AutoRTFM::Testing::Commit([&]
		{
			TestType* Ptr = new TestType(42, &bWasDestructed);

			Ptr->AddRef();
			Ptr->AddRef();
			Ptr->Release();
			Ptr->AddRef();
			Ptr->Release();
			Ptr->Release();
			REQUIRE(!bWasDestructed);
		});

		REQUIRE(bWasDestructed);
	}
}

TEMPLATE_TEST_CASE("PreviouslyAllocated", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt, FQueryableRefCountedInt)
{
	SECTION("Using new T()")
	{
		bool bWasDestructed = false;

		{
			TRefCountPtr<TestType> Foo(new TestType(42, &bWasDestructed));

			AutoRTFM::Testing::Commit([&]
			{
				// Make a copy to bump the reference count.
				TRefCountPtr<TestType> Copy = Foo;
				Copy->Value = 13;
			});

			REQUIRE(Foo->Value == 13);
			REQUIRE(!bWasDestructed);
		}

		REQUIRE(bWasDestructed);
	}

	SECTION("Using MakeRefCount<T>")
	{
		bool bWasDestructed = false;

		{
			TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42, &bWasDestructed);

			AutoRTFM::Testing::Commit([&]
			{
				// Make a copy to bump the reference count.
				TRefCountPtr<TestType> Copy = Foo;
				Copy->Value = 13;
			});

			REQUIRE(Foo->Value == 13);
			REQUIRE(!bWasDestructed);
		}

		REQUIRE(bWasDestructed);
	}
}

TEMPLATE_TEST_CASE("AbortWithPreviouslyAllocated", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt, FQueryableRefCountedInt)
{
	SECTION("Using new T()")
	{
		bool bWasDestructed = false;

		{
			TRefCountPtr<TestType> Foo(new TestType(42, &bWasDestructed));

			AutoRTFM::Testing::Abort([&]
			{
				// Make a copy to bump the reference count.
				TRefCountPtr<TestType> Copy = Foo;
				Copy->Value = 13;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(Foo->Value == 42);
			REQUIRE(!bWasDestructed);
		}

		REQUIRE(bWasDestructed);
	}

	SECTION("Using MakeRefCount<T>")
	{
		bool bWasDestructed = false;

		{
			TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42, &bWasDestructed);

			AutoRTFM::Testing::Abort([&]
			{
				// Make a copy to bump the reference count.
				TRefCountPtr<TestType> Copy = Foo;
				Copy->Value = 13;

				AutoRTFM::AbortTransaction();
			});

			REQUIRE(Foo->Value == 42);
			REQUIRE(!bWasDestructed);
		}

		REQUIRE(bWasDestructed);
	}
}

TEMPLATE_TEST_CASE("AbortWithNewlyAllocated", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt, FQueryableRefCountedInt)
{
	int Result = 42;

	SECTION("Using new T()")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TRefCountPtr<TestType> Foo(new TestType(42));
			Result = Foo->Value;
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Using new T() and a copy")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TRefCountPtr<TestType> Foo(new TestType(42));
			TRefCountPtr<TestType> Copy = Foo;
			Result = Copy->Value;
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Using MakeRefCount<T>")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42);
			Result = Foo->Value;
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Using MakeRefCount<T> and a copy")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42);
			TRefCountPtr<TestType> Copy = Foo;
			Result = Copy->Value;
			AutoRTFM::AbortTransaction();
		});
	}

	REQUIRE(Result == 42);
}

TEMPLATE_TEST_CASE("OnCommitCapturingRefCountPtr", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt, FQueryableRefCountedInt)
{
	TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42);

	SECTION("Committing")
	{
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnCommit([Foo]
			{
				Foo->Value = 13;
			});
		});

		REQUIRE(Foo->Value == 13);
	}

	SECTION("Aborting")
	{
		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnCommit([Foo]
			{
				Foo->Value = 13;
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Foo->Value == 42);
	}
}

TEMPLATE_TEST_CASE("OnAbortCapturingRefCountPtr", "[RefCountPtr]", FRefCountedInt, FRefCountedMixinInt, FThreadSafeRefCountedInt, FQueryableRefCountedInt)
{
	TRefCountPtr<TestType> Foo = MakeRefCount<TestType>(42);

	SECTION("Committing")
	{
		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::OnAbort([Foo]
			{
				Foo->Value = 13;
			});
		});

		// The test harness can decide to abort transactions if `ShouldRetryNonNestedTransactions` is set.
		REQUIRE((Foo->Value == 42 || AutoRTFM::ForTheRuntime::ShouldRetryNonNestedTransactions()));
	}

	SECTION("Aborting")
	{
		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::OnAbort([Foo]
			{
				Foo->Value = 13;
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Foo->Value == 13);
	}
}
