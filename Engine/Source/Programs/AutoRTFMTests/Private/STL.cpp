// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

#include <math.h>
#include <array>
#include <utility>

TEST_CASE("STL.swap")
{
	auto TestSwap = []<typename T>(T A, T B)
    {
        T const OldA = A;
        T const OldB = B;

        AutoRTFM::Testing::Abort([&]
        {
            std::swap(A, B);
            REQUIRE(A == OldB);
            REQUIRE(B == OldA);
            AutoRTFM::AbortTransaction();
        });
        REQUIRE(A == OldA);
        REQUIRE(B == OldB);

        AutoRTFM::Testing::Commit([&]
        {
            std::swap(A, B);
            REQUIRE(A == OldB);
            REQUIRE(B == OldA);
        });
        REQUIRE(A == OldB);
        REQUIRE(B == OldA);
    };
    TestSwap.operator()<char>(1, 2);
    TestSwap.operator()<int>(1, 2);
    TestSwap.operator()<float>(1, 2);
    TestSwap.operator()<const char*>("A", "B");
    TestSwap.operator()<std::string>("A", "B");
    TestSwap.operator()<std::array<int, 3>>({1,2,3}, {4,5,6});
}
