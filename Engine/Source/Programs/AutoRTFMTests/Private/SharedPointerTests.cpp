// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "HAL/UnrealMemory.h"
#include "Templates/SharedPointer.h"

TEST_CASE("SharedPointer.PreviouslyAllocated")
{
	TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(42));

	AutoRTFM::Testing::Commit([&]
		{
			// Make a copy to bump the reference count.
			TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;

			*Copy = 13;
		});

	REQUIRE(13 == *Foo);
	REQUIRE(1 == Foo.GetSharedReferenceCount());
}

TEST_CASE("SharedPointer.AbortWithPreviouslyAllocated")
{
	TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(42));

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
		{
			// Make a copy to bump the reference count.
			TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;

			*Copy = 13;

			AutoRTFM::AbortTransaction();
		}));

	REQUIRE(42 == *Foo);
	REQUIRE(1 == Foo.GetSharedReferenceCount());
}

TEST_CASE("SharedPointer.NewlyAllocated")
{
	int Copy = 42;

	AutoRTFM::Testing::Commit([&]
		{
			TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(13));

			Copy = *Foo;
		});

	REQUIRE(13 == Copy);
}

TEST_CASE("SharedPointer.AbortWithNewlyAllocated")
{
	int Result = 42;

	REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
		{
			TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(13));
			TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;
			Result = *Copy;
			AutoRTFM::AbortTransaction();
		}));

	REQUIRE(42 == Result);
}

TEST_CASE("SharedPointer.NestedTransactionWithPreviouslyAllocated")
{
	TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(42));

	AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Commit([&]
				{
					// Make a copy to bump the reference count.
					TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;

					*Copy = 13;
				});
		});

	REQUIRE(13 == *Foo);
	REQUIRE(1 == Foo.GetSharedReferenceCount());
}

TEST_CASE("SharedPointer.AbortNestedTransactionWithPreviouslyAllocated")
{
	TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(42));

	AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
				{
					// Make a copy to bump the reference count.
					TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;

					*Copy = 13;

					AutoRTFM::AbortTransaction();
				}));
		});

	REQUIRE(42 == *Foo);
	REQUIRE(1 == Foo.GetSharedReferenceCount());
}

TEST_CASE("SharedPointer.NestedTransactionWithNewlyAllocated")
{
	int Result = 42;

	AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Commit([&]
				{
					TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(13));
					TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;
					Result = *Copy;
				});
		});

	REQUIRE(13 == Result);
}

TEST_CASE("SharedPointer.AbortNestedTransactionWithNewlyAllocated")
{
	int Result = 42;

	AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
				{
					TSharedPtr<int, ESPMode::ThreadSafe> Foo(new int(13));
					TSharedPtr<int, ESPMode::ThreadSafe> Copy = Foo;
					Result = *Copy;
					AutoRTFM::AbortTransaction();
				}));
		});

	REQUIRE(42 == Result);
}

template<typename T> FORCENOINLINE void* MakeMemoryForT()
{
	return FMemory::Malloc(sizeof(T));
}

TEST_CASE("SharedPointer.NestedTransactionWithPlacementNewlyAllocated")
{
	int Result = 42;

	AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Testing::Commit([&]
				{
					using Ptr = TSharedPtr<int, ESPMode::ThreadSafe>;
					void* const Memory = MakeMemoryForT<Ptr>();
					Ptr* const Foo = new (Memory) Ptr(new int(13));
					Ptr Copy = *Foo;
					Result = *Copy;
					reinterpret_cast<Ptr*>(Foo)->~Ptr();
					FMemory::Free(Memory);
				});
		});

	REQUIRE(13 == Result);
}

TEST_CASE("SharedPointer.AbortNestedTransactionWithPlacementNewlyAllocated")
{
	int Result = 42;

	AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(AutoRTFM::ETransactionResult::AbortedByRequest == AutoRTFM::Transact([&]
				{
					void* const Memory = MakeMemoryForT<TSharedPtr<int, ESPMode::ThreadSafe>>();
					TSharedPtr<int, ESPMode::ThreadSafe>* const Foo = new (Memory) TSharedPtr<int, ESPMode::ThreadSafe>(new int(13));
					TSharedPtr<int, ESPMode::ThreadSafe> Copy = *Foo;
					Result = *Copy;
					AutoRTFM::AbortTransaction();
				}));
		});

	REQUIRE(42 == Result);
}

TEST_CASE("SharedPointer.OnCommitCapturesSharedPtr")
{
	AutoRTFM::Transact([&]
	{
		TSharedPtr<int> Shared{new int};
		AutoRTFM::OnCommit([Shared] {});
		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("SharedPointer.OnCommitCapturesWeakPtr")
{
	AutoRTFM::Transact([&]
	{
		TSharedPtr<int> Shared{new int};
		TWeakPtr<int> Weak{Shared};
		AutoRTFM::OnCommit([Weak] {});
		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("SharedPointer.OnAbortCapturesSharedPtr")
{
	AutoRTFM::Transact([&]
	{
		TSharedPtr<int> Shared{new int};
		AutoRTFM::OnAbort([Shared] {});
		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("SharedPointer.OnAbortCapturesWeakPtr")
{
	AutoRTFM::Transact([&]
	{
		TSharedPtr<int> Shared{new int};
		TWeakPtr<int> Weak{Shared};
		AutoRTFM::OnAbort([Weak] {});
		AutoRTFM::AbortTransaction();
	});
}

TEST_CASE("SharedPointer.ReturnFromOpen")
{
	TSharedPtr<int> Shared = AutoRTFM::Open([]
	{
		return TSharedPtr<int>{new int(42)};
	});

	REQUIRE(*Shared == 42);
}

TEST_CASE("SharedReference.ReturnFromOpen")
{
	TSharedRef<int> Shared = AutoRTFM::Open([]
	{
		return MakeShared<int>(42);
	});

	REQUIRE(*Shared == 42);
}

TEST_CASE("WeakPointer.ReturnFromOpen")
{
	TSharedPtr<int> Shared = TSharedPtr<int>{new int(42)};
	TWeakPtr<int> Weak = AutoRTFM::Open([&]
	{
		return TWeakPtr<int>{Shared};
	});

	REQUIRE(*Weak.Pin() == 42);
}
