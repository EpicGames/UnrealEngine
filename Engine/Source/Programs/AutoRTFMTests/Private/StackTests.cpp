// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "Stack.h"
#include "Catch2Includes.h"
#include "ObjectLifetimeHelper.h"

using TrivialStack = AutoRTFM::TStack<int, 4>;
using NonTrivialStack = AutoRTFM::TStack<AutoRTFMTestUtils::FObjectLifetimeHelper, 4>;

TEMPLATE_TEST_CASE("Stack", "", TrivialStack, NonTrivialStack)
{
	AutoRTFM::UnreachableIfClosed();

	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == 0);
	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls == 0);

	using StackType = TestType;
	using ElementType = typename TestType::ElementType;

	auto Check = [&] AUTORTFM_DISABLE (StackType& Stack, std::vector<ElementType> Expected)
	{
		// Check the reported count is as expected.
		REQUIRE(Stack.Num() == Expected.size());
		REQUIRE(Stack.IsEmpty() == Expected.empty());

		if (!Expected.empty())
		{
			REQUIRE(Expected.back() == Stack.Back());
			REQUIRE(Expected.front() == Stack.Front());
			
			// Check the Front() and Back() return a mutable reference.
			Stack.Front() = 99;
			REQUIRE(Stack.Front() == 99);
			Stack.Front() = Expected.front();

			Stack.Back() = 99;
			REQUIRE(Stack.Back() == 99);
			Stack.Back() = Expected.back();
		}

		for (size_t I = 0; I < Expected.size(); I++)
		{
			// Check the element is as expected.
			REQUIRE(Stack[I] == Expected[I]);
			// Check the index operator returns a mutable reference.
			Stack[I] = 99;
			REQUIRE(Stack[I] == 99);
			Stack[I] = Expected[I];
		}
		// Check the non-const begin() / end().
		{
			size_t I = 0;
			for (ElementType& Item : Stack)
			{
				REQUIRE(Item == Expected[I]);
				I++;
			}
			REQUIRE(I == Expected.size());
		}
		// Check the const begin() / end().
		{
			size_t I = 0;
			for (const ElementType& Item : const_cast<const StackType&>(Stack))
			{
				REQUIRE(Item == Expected[I]);
				I++;
			}
			REQUIRE(I == Expected.size());
		}
	};

	SECTION("Push, Pop")
	{
		StackType Stack;
		Check(Stack, {});

		Stack.Push(1);
		ElementType* InlineAddress = &Stack[0];
		Check(Stack, {1});

		Stack.Push(2);
		REQUIRE(&Stack[0] == InlineAddress);
		Check(Stack, {1, 2});

		Stack.Push(3);
		REQUIRE(&Stack[0] == InlineAddress);
		Check(Stack, {1, 2, 3});

		Stack.Pop();
		REQUIRE(&Stack[0] == InlineAddress);
		Check(Stack, {1, 2});

		Stack.Push(4);
		REQUIRE(&Stack[0] == InlineAddress);
		Check(Stack, {1, 2, 4});

		Stack.Push(5);
		REQUIRE(&Stack[0] == InlineAddress);
		Check(Stack, {1, 2, 4, 5});

		Stack.Push(6); // Spill inline -> heap
		REQUIRE(&Stack[0] != InlineAddress);
		Check(Stack, {1, 2, 4, 5, 6});

		Stack.Pop();
		REQUIRE(&Stack[0] != InlineAddress);
		Check(Stack, {1, 2, 4, 5});
	}

	SECTION("PushAll")
	{
		StackType Target;
		// PushAll() with empty target
		{
			StackType Source;
			Source.Push(1);
			Source.Push(2);
			Target.PushAll(std::move(Source));
			REQUIRE(Source.IsEmpty());
			Check(Target, {1, 2});
		}

		// PushAll() with target holding inline data
		{
			StackType Source;
			Source.Push(30);
			Source.Push(40);
			Target.PushAll(std::move(Source));
			REQUIRE(Source.IsEmpty());
			Check(Target, {1, 2, 30, 40});
		}

		// PushAll() with target spilling from inline -> heap.
		{
			StackType Source;
			Source.Push(500);
			Source.Push(600);
			Source.Push(700);
			Target.PushAll(std::move(Source));
			REQUIRE(Source.IsEmpty());
			Check(Target, {1, 2, 30, 40, 500, 600, 700});
		}
	}

	SECTION("Clear / Reset")
	{
		StackType Stack;
		Stack.Push(1);
		ElementType* InlineAddress = &Stack[0];
		Stack.Push(2);
		Stack.Push(3);
		Stack.Push(4);
		Stack.Push(5);
		SECTION("Clear")
		{
			Stack.Clear();
			Check(Stack, {});
			Stack.Push(100);
			Check(Stack, {100});
			REQUIRE(&Stack[0] != InlineAddress);
		}
		SECTION("Reset")
		{
			Stack.Reset();
			Check(Stack, {});
			Stack.Push(100);
			Check(Stack, {100});
			REQUIRE(&Stack[0] == InlineAddress);
		}
	}

	SECTION("Copy Construct")
	{
		SECTION("Inline")
		{
			StackType Source;
			Source.Push(1);
			Source.Push(2);
			Source.Push(3);
			StackType Target(Source);
			Check(Source, {1, 2, 3});
			Check(Target, {1, 2, 3});
		}

		SECTION("Heap")
		{
			StackType Source;
			Source.Push(1);
			Source.Push(2);
			Source.Push(3);
			Source.Push(4);
			Source.Push(5);
			StackType Target(Source);
			Check(Source, {1, 2, 3, 4, 5});
			Check(Target, {1, 2, 3, 4, 5});
		}
	}

	SECTION("Move Construct")
	{
		SECTION("Inline")
		{
			StackType Source;
			Source.Push(1);
			Source.Push(2);
			Source.Push(3);
			StackType Target(std::move(Source));
			Check(Source, {});
			Check(Target, {1, 2, 3});
		}

		SECTION("Heap")
		{
			StackType Source;
			Source.Push(1);
			Source.Push(2);
			Source.Push(3);
			Source.Push(4);
			Source.Push(5);
			StackType Target(std::move(Source));
			Check(Source, {});
			Check(Target, {1, 2, 3, 4, 5});
		}
	}

	SECTION("Copy Assign")
	{
		StackType Source;
		StackType Target;
		auto CheckCopy = [&] AUTORTFM_DISABLE
		{
			SECTION("Source Empty")
			{
				Target = Source;
				Check(Source, {});
				Check(Target, {});
			}
			SECTION("Source Inline")
			{
				Source.Push(1);
				Source.Push(2);
				Source.Push(3);
				Target = Source;
				Check(Source, {1, 2, 3});
				Check(Target, {1, 2, 3});
			}
			SECTION("Source Heap")
			{
				Source.Push(1);
				Source.Push(2);
				Source.Push(3);
				Source.Push(4);
				Source.Push(5);
				Target = Source;
				Check(Source, {1, 2, 3, 4, 5});
				Check(Target, {1, 2, 3, 4, 5});
			}
		};

		SECTION("Target Empty")
		{
			CheckCopy();
		}
		SECTION("Target Inline")
		{
			Target.Push(10);
			Target.Push(20);
			Target.Push(30);
			CheckCopy();
		}
		SECTION("Target Heap")
		{
			Target.Push(10);
			Target.Push(20);
			Target.Push(30);
			Target.Push(40);
			Target.Push(50);
			CheckCopy();
		}
	}

	SECTION("Copy Assign Self")
	{
		StackType Stack;
		StackType& Alias = Stack;
		SECTION("Empty")
		{
			Stack = Alias;
			Check(Stack, {});
		}
		SECTION("Inline")
		{
			Stack.Push(1);
			Stack.Push(2);
			Stack.Push(3);
			Stack = Alias;
			Check(Stack, {1, 2, 3});
		}
		SECTION("Heap")
		{
			Stack.Push(1);
			Stack.Push(2);
			Stack.Push(3);
			Stack.Push(4);
			Stack.Push(5);
			Stack = Alias;
			Check(Stack, {1, 2, 3, 4, 5});
		}
	}

	SECTION("Move Assign")
	{
		StackType Source;
		StackType Target;
		auto CheckMove = [&] AUTORTFM_DISABLE
		{
			SECTION("Source Empty")
			{
				Target = std::move(Source);
				Check(Source, {});
				Check(Target, {});
			}
			SECTION("Source Inline")
			{
				Source.Push(1);
				Source.Push(2);
				Source.Push(3);
				Target = std::move(Source);
				Check(Source, {});
				Check(Target, {1, 2, 3});
			}
			SECTION("Source Heap")
			{
				Source.Push(1);
				Source.Push(2);
				Source.Push(3);
				Source.Push(4);
				Source.Push(5);
				Target = std::move(Source);
				Check(Source, {});
				Check(Target, {1, 2, 3, 4, 5});
			}
		};

		SECTION("Target Empty")
		{
			CheckMove();
		}
		SECTION("Target Inline")
		{
			Target.Push(10);
			Target.Push(20);
			Target.Push(30);
			CheckMove();
		}
		SECTION("Target Heap")
		{
			Target.Push(10);
			Target.Push(20);
			Target.Push(30);
			Target.Push(40);
			Target.Push(50);
			CheckMove();
		}
	}

	SECTION("Move Assign Self")
	{
		StackType Stack;
		StackType& Alias = Stack;
		SECTION("Empty")
		{
			Stack = std::move(Alias);
			Check(Stack, {});
		}
		SECTION("Inline")
		{
			Stack.Push(1);
			Stack.Push(2);
			Stack.Push(3);
			Stack = std::move(Alias);
			Check(Stack, {1, 2, 3});
		}
		SECTION("Heap")
		{
			Stack.Push(1);
			Stack.Push(2);
			Stack.Push(3);
			Stack.Push(4);
			Stack.Push(5);
			Stack = std::move(Alias);
			Check(Stack, {1, 2, 3, 4, 5});
		}
	}

	SECTION("Soak")
	{
		StackType Stack;
		std::vector<ElementType> Expected;
		for (unsigned I = 0; I < 10000; I++)
		{
			unsigned Mod100 = (I * 15485863) % 100;
			switch (Mod100)
			{
				case 0:
				{
					Stack.Clear();
					Expected.clear();
					break;
				}
				case 1:
				{	
					Stack.Reset();
					Expected.clear();
					break;
				}
				case 2:
				{	
					StackType Copy(Stack);
					Stack = Copy;
					break;
				}
				case 3:
				{	
					StackType Move(std::move(Stack));
					Stack = std::move(Move);
					break;
				}
				default:
				{
					if (Mod100 > 40 || Expected.empty())
					{
						Stack.Push(I);
						Expected.push_back(I);
						Check(Stack, Expected);
					}
					else
					{
						Stack.Pop();
						Expected.pop_back();
						Check(Stack, Expected);
					}
					break;
				}
			}
		}
	}

	REQUIRE(AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls == AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls);
	AutoRTFMTestUtils::FObjectLifetimeHelper::ConstructorCalls = 0;
	AutoRTFMTestUtils::FObjectLifetimeHelper::DestructorCalls = 0;
}
