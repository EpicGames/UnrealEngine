// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Defines.h"
#include "AutoRTFM/Testing.h"
#include "Context.h"
#include "ContextInlines.h"
#include "Transaction.h"
#include "Catch2Includes.h"
#include "CrossCUTests.h"
#include "Utils.h"

#include <chrono>
#include <condition_variable>
#include <mutex>
#include <thread>

TEST_CASE("Statics.Int")
{
	auto GetAndIncrement = [] AUTORTFM_ENABLE
		{
			static int Thing = 42;
			return Thing++;
		};

	REQUIRE(
		AutoRTFM::ETransactionResult::Committed ==
		AutoRTFM::Transact([&]()
			{
				GetAndIncrement();
			}));

	// The transactional effect of incrementing the static will have
	// been committed, and since we are accessing the exact same
	// static we should see its side effects.
	REQUIRE(43 == GetAndIncrement());
}

TEST_CASE("Statics.IntAbort")
{
	auto GetAndIncrement = [] AUTORTFM_ENABLE
		{
			static int Thing = 42;
			return Thing++;
		};

    REQUIRE(
        AutoRTFM::ETransactionResult::AbortedByRequest ==
        AutoRTFM::Transact([&] ()
        {
			if (42 == GetAndIncrement())
			{
				AutoRTFM::AbortTransaction();
			}
        }));
    
	// The transactional effect of incrementing the static will have
	// been rolled back, but it should still be initialized correctly.
	REQUIRE(42 == GetAndIncrement());
}

struct SomeStruct
{
	int Payload[42];
	int Current = 0;
};

TEST_CASE("Statics.Struct")
{
	auto GetSlot = [] AUTORTFM_ENABLE
		{
			static SomeStruct S;
			int* const Result = &S.Payload[S.Current];
			*Result = ++S.Current;
			return Result;
		};

	REQUIRE(
		AutoRTFM::ETransactionResult::Committed ==
		AutoRTFM::Transact([&]()
			{
				int* const Slot = GetSlot();
				*Slot = 13;
			}));

	// The transactional effect of incrementing the static will have
	// been committed, so we should see the side effects.
	int* const Slot = GetSlot();
	REQUIRE(2 == *Slot);

	// The transaction would have written to the previous slot.
	REQUIRE(13 == Slot[-1]);
}

TEST_CASE("Statics.StructAbort")
{
	auto GetSlot = [] AUTORTFM_ENABLE
		{
			static SomeStruct S;
			int* const Result = &S.Payload[S.Current];
			*Result = ++S.Current;
			return Result;
		};

	REQUIRE(
		AutoRTFM::ETransactionResult::AbortedByRequest ==
		AutoRTFM::Transact([&]()
			{
				int* const Slot = GetSlot();
				*Slot = 13;
				AutoRTFM::AbortTransaction();
			}));

	// The transactional effect of incrementing the static will have
	// been rolled back, but it should still be initialized correctly.
	REQUIRE(1 == *GetSlot());
}

using FuncPtrType = bool(*)();

FuncPtrType GIsClosed = &AutoRTFM::IsClosed;
FuncPtrType GIsTransactional = &AutoRTFM::IsTransactional;

FORCENOINLINE static bool GetIsClosed()
{
	return AutoRTFM::IsClosed();
}

FORCENOINLINE static bool GetIsTransactional()
{
	return AutoRTFM::IsTransactional();
}

TEST_CASE("Static.IsClosed")
{
	SECTION("Normal")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasClosedAtConstruction = AutoRTFM::IsClosed();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasClosedAtConstruction);
		});
	}

	SECTION("FromGlobal")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasClosedAtConstruction = GIsClosed();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasClosedAtConstruction);
		});
	}

	SECTION("InNoInlineCall")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasClosedAtConstruction = GetIsClosed();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasClosedAtConstruction);
		});
	}

	SECTION("InNestedStaticLocalInitializer")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			struct MyInner final
			{
				bool bWasClosedAtConstruction = AutoRTFM::IsClosed();
			};

			MyStruct()
			{
				static MyInner Mine;
				bWasClosedAtConstruction = Mine.bWasClosedAtConstruction;
			}

			bool bWasClosedAtConstruction;
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasClosedAtConstruction);
		});
	}

	SECTION("InOpenStaticLocalInitializer")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasClosedAtConstruction = AutoRTFM::IsClosed();
		};

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				static MyStruct Mine;
				REQUIRE(false == Mine.bWasClosedAtConstruction);
			});
		});
	}
}

TEST_CASE("Static.IsTransactional")
{
	SECTION("Normal")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasTransactionalAtConstruction = AutoRTFM::IsTransactional();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasTransactionalAtConstruction);
		});
	}

	SECTION("FromGlobal")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasTransactionalAtConstruction = GIsTransactional();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasTransactionalAtConstruction);
		});
	}

	SECTION("InNoInlineCall")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasTransactionalAtConstruction = GetIsTransactional();
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasTransactionalAtConstruction);
		});
	}

	SECTION("InNestedStaticLocalInitializer")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			struct MyInner final
			{
				bool bWasTransactionalAtConstruction = AutoRTFM::IsTransactional();
			};

			MyStruct()
			{
				static MyInner Mine;
				bWasTransactionalAtConstruction = Mine.bWasTransactionalAtConstruction;
			}

			bool bWasTransactionalAtConstruction;
		};

		AutoRTFM::Testing::Commit([&]
		{
			static MyStruct Mine;
			REQUIRE(false == Mine.bWasTransactionalAtConstruction);
		});
	}

	SECTION("InOpenStaticLocalInitializer")
	{
		struct AUTORTFM_ENABLE MyStruct final
		{
			bool bWasTransactionalAtConstruction = AutoRTFM::IsTransactional();
		};

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				static MyStruct Mine;
				REQUIRE(false == Mine.bWasTransactionalAtConstruction);
			});
		});
	}
}

// The following code is a reduced version of the bug which would trigger SOL-7447.
namespace 
{
namespace SOL7447
{

struct MyString
{
	MyString() = default;
	~MyString() 
	{ 
		// This is only here so that the compiler won't optimize the destructor away.
		// You can also reproduce by writing through a volatile or to a global variable.
		CrossCU::SomeFunction(Char);
	}

	char Char = 0;
};

struct FFormattingRules
{
	char NegativePrefixString;
	char NegativeSuffixString;
	char PositivePrefixString;
	char PositiveSuffixString;
};

class FSigningStrings
{
public:
	FSigningStrings(const FFormattingRules& InFormattingRules)
	{
		// Resolve out the default cases
		if (InFormattingRules.NegativePrefixString)
		{
			NegativePrefixStringPtr = &InFormattingRules.NegativePrefixString;
		}
		if (InFormattingRules.NegativeSuffixString)
		{
			NegativeSuffixStringPtr = &InFormattingRules.NegativeSuffixString;
		}
		if (InFormattingRules.PositivePrefixString)
		{
			PositivePrefixStringPtr = &InFormattingRules.PositivePrefixString;
		}
		if (InFormattingRules.PositiveSuffixString)
		{
			PositiveSuffixStringPtr = &InFormattingRules.PositiveSuffixString;
		}
	}

	const char& GetNegativePrefixString() const
	{
		return NegativePrefixStringPtr ? *NegativePrefixStringPtr : GetEmptyString().Char;
	}

	const char& GetNegativeSuffixString() const
	{
		return NegativeSuffixStringPtr ? *NegativeSuffixStringPtr : GetEmptyString().Char;
	}

	const char& GetPositivePrefixString() const
	{
		return PositivePrefixStringPtr ? *PositivePrefixStringPtr : GetEmptyString().Char;
	}

	const char& GetPositiveSuffixString() const
	{
		return PositiveSuffixStringPtr ? *PositiveSuffixStringPtr : GetEmptyString().Char;
	}

private:
	const char* NegativePrefixStringPtr = nullptr;
	const char* NegativeSuffixStringPtr = nullptr;
	const char* PositivePrefixStringPtr = nullptr;
	const char* PositiveSuffixStringPtr = nullptr;

	const MyString& GetEmptyString() const
	{
		static const MyString EmptyStr = MyString();
		return EmptyStr;
	}
};

TPair<char, char> BuildFinalString(const FFormattingRules& InFormattingRules, bool bIsNegative) 
{
	const FSigningStrings SigningStrings(InFormattingRules);
	const char& FinalPrefixStr = bIsNegative ? SigningStrings.GetNegativePrefixString() : SigningStrings.GetPositivePrefixString();
 	const char& FinalSuffixStr = bIsNegative ? SigningStrings.GetNegativeSuffixString() : SigningStrings.GetPositiveSuffixString();

	return {FinalPrefixStr, FinalSuffixStr};
}

} // namespace SOL7447
} // namespace

TEST_CASE("Static.ComplexStaticInitialization")
{
	AutoRTFM::Testing::Commit([]
	{
		SOL7447::FFormattingRules Rules{'+', 0, '-', 0};
		TPair<char, char> A = SOL7447::BuildFinalString(Rules, false);
		TPair<char, char> B = SOL7447::BuildFinalString(Rules, true);

		REQUIRE(A != B);
	});
}
