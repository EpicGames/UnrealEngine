// Copyright Epic Games, Inc. All Rights Reserved.

#include "Catch2Includes.h"
#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"

#include "Async/TaskGraphInterfaces.h"
#include "Stats/Stats.h"
#include "Stats/StatsCommand.h"
#include "Stats/StatsData.h"
#include "Stats/StatsSystem.h"
#include "Stats/StatsSystemTypes.h"

#if STATS

DECLARE_STATS_GROUP(TEXT("AutoRTFM Tests"), STATGROUP_AutoRTFM_Tests, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Standard Cycle Counter"), STAT_StandardCycleCounter, STATGROUP_AutoRTFM_Tests);
// Ones we do not have tests for:
// FScopeCycleCounterEmitter // Requires TemplateObj->SpriteTemplate->GetStatID();
// FScopeCycleCounterSWidget // Requires SWidget
// FScopedTraceSolverCallback // Requires ISimCallbackObject
// FScopeCycleCounterUObject // Requires UObject

// This will put an Add message directly into the Thread Stats
// Certain stats can perform this internally, we're just simulating that with an existing Stat.
static void AddDummyStatsMessage()
{
	// Keep track of how long we were waiting for it
	const FName StatName = GET_STATFNAME(STAT_StandardCycleCounter);
	const uint64 WaitCycles = 0;
	FThreadStats::AddMessage(StatName, EStatOperation::Add, WaitCycles, true);
}

// Make sure that the stats advance a frame and that we have its data available on the game thread
static void TickStatsAndWaitForFrameToBecomeAvailable()
{
	// Advance the Stats frame forward.
	FThreadStats::ExplicitFlush();
	UE::Stats::FStats::TickCommandletStats();

	// Go through this cycle a bunch of times in case messages are passed back and forth
	for (int Index = 0; Index < 5; ++Index)
	{
		// Flush so that all messages are assigned the last frame and sent to Stats Thread.
		FThreadStats::ExplicitFlush();
		// Wait for the Stats Thread to receive the Flush & Process. Sends a message back to Game Thread.
		FThreadStats::WaitForStats();

		// Now process that message bound for the Game Thread
		// This includes the update of FLatestGameThreadStatsData 
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	}
}

// Return the accumulated value of the stat (should always be increasing)
static int64 GetAutoRtfmStatValue()
{
	FGameThreadStatsData* ViewDataPtr = FLatestGameThreadStatsData::Get().Latest;
	if (ViewDataPtr)
	{
		static const FName StatName = TEXT("STAT_StandardCycleCounter");
		if (const FComplexStatMessage* StatMessage = ViewDataPtr->GetStatData(StatName))
		{
			return StatMessage->GetValue_Duration(EComplexStatField::IncSum);
		}
	}

	return -1.0f;
}

TEST_CASE("StatsTests")
{
	SECTION("Enable AutoRTFM Stats Gathering")
	{
		// Try and register FHUDGroupManager::NewFrame but this may not work
		// Because we may be running with Stats already enabled (and that registration only happens
		// when we first enable Stats).  Attempt it anyway.
		UE::Stats::DirectStatsCommand(TEXT("stat AutoRTFM_Tests"), /*bBlockForCompletion=*/true);

		// Let's fill the stats history
		for (int Num = 0; Num < 10; ++Num)
		{
			TickStatsAndWaitForFrameToBecomeAvailable();
		}
	}

	SECTION("Stats FScopedCycleCounter Outside Transaction")
	{
		SCOPE_CYCLE_COUNTER(STAT_StandardCycleCounter);
	}

	SECTION("Stats FScopedCycleCounter Inside Transaction")
	{
		int64 PreviousValue = GetAutoRtfmStatValue();
		AutoRTFM::Testing::Abort([&]
		{
			SCOPE_CYCLE_COUNTER(STAT_StandardCycleCounter);
			AutoRTFM::AbortTransaction();
		});

		TickStatsAndWaitForFrameToBecomeAvailable();

		int64 NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
		PreviousValue = NewValue;

		AutoRTFM::Testing::Commit([&]
		{
			SCOPE_CYCLE_COUNTER(STAT_StandardCycleCounter);
		});

		TickStatsAndWaitForFrameToBecomeAvailable();
		NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
	}

	SECTION("Stats Multi-FScopedCycleCounter Re-Entrant w/ Transaction")
	{
		auto ChainCall = [] AUTORTFM_ENABLE(TFunctionRef<void()> ChainedFunc)
		{
			SCOPE_CYCLE_COUNTER(STAT_StandardCycleCounter);
			ChainedFunc();
		};

		int64 PreviousValue = GetAutoRtfmStatValue();
		AutoRTFM::Testing::Abort([&]
		{
			ChainCall([&]
			{
				ChainCall([&]
				{
					AutoRTFM::AbortTransaction();
				});
			});
		});

		TickStatsAndWaitForFrameToBecomeAvailable();
		int64 NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
		PreviousValue = NewValue;

		AutoRTFM::Testing::Commit([&]
		{
			ChainCall([&]
			{
				ChainCall([&]
				{
				});
			});
		});

		TickStatsAndWaitForFrameToBecomeAvailable();
		NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
	}

	SECTION("Stats FScopedCycleCounter w/ Manual Counter")
	{
		auto ChainCall = [] AUTORTFM_ENABLE(TFunctionRef<void()> ChainedFunc)
		{
			SCOPE_CYCLE_COUNTER(STAT_StandardCycleCounter);
			ChainedFunc();
		};

		int64 PreviousValue = GetAutoRtfmStatValue();
		AutoRTFM::Testing::Abort([&]
		{
			ChainCall([&]
			{
				ChainCall([&]
				{
					AddDummyStatsMessage();
					AutoRTFM::AbortTransaction();
				});
			});
		});

		TickStatsAndWaitForFrameToBecomeAvailable();
		int64 NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
		PreviousValue = NewValue;

		AutoRTFM::Testing::Commit([&]
		{
			ChainCall([&]
			{
				ChainCall([&]
				{
					AddDummyStatsMessage();
				});
			});
		});

		TickStatsAndWaitForFrameToBecomeAvailable();
		NewValue = GetAutoRtfmStatValue();
		CATCH_CHECK(NewValue > PreviousValue);
	}
}

#endif
