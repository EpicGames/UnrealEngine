// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskArray.h"
#include "Catch2Includes.h"

#include <memory>
#include <random>
#include <unordered_map>

TEST_CASE("TTaskArray")
{
	std::mt19937 Rand(0x1234);
	using TaskType = int;
	using TaskArrayType = AutoRTFM::TTaskArray
		<
			/* TaskType */ int,
			/* KeyType */ const void*
		>;
	using TaskPoolType = typename TaskArrayType::FEntryPool;
	TaskPoolType Pool;
	auto TaskArray = std::make_unique<TaskArrayType>(Pool);

	struct FEntry
	{
		int Value = 0;
		FEntry* Prev = nullptr;
		FEntry* Next = nullptr;
	};

	std::vector<void*> Keys;
	std::unordered_map<void*, std::vector<FEntry*>> EntriesByKey;
	FEntry* Head = nullptr;
	FEntry* Tail = nullptr;

	auto DeleteEntry = [&](FEntry* Entry)
	{
		if (Head == Entry) { Head = Entry->Next; }
		if (Tail == Entry) { Tail = Entry->Prev; }
		if (Entry->Prev) { Entry->Prev->Next = Entry->Next; }
		if (Entry->Next) { Entry->Next->Prev = Entry->Prev; }
		Entry->Prev = nullptr;
		Entry->Next = nullptr;
		delete Entry;
	};

	struct FConfig
	{
		size_t NumKeys = 0;
		size_t NumElements = 0;
	};

	auto Populate = [&](FConfig Config)
	{
		for (size_t I = 0; I < Config.NumKeys; I++)
		{
			void* Key = reinterpret_cast<void*>(static_cast<uintptr_t>(I + 1) << 20);
			Keys.push_back(Key);
		}
		
		for (size_t I = 0; I < Config.NumElements; I++)
		{
			const int Value = static_cast<int>(Rand());
			FEntry* const Entry = new FEntry{};
			Entry->Value = Value;
			if (!Head)
			{ 
				Head = Entry;
			}
			else
			{
				Tail->Next = Entry;
				Entry->Prev = Tail;
			}
			Tail = Entry;
			if (Config.NumKeys > 0 && Rand() & 1)
			{
				void* Key = Keys[Rand() % Keys.size()];
				EntriesByKey[Key].push_back(Entry);
				TaskArray->AddKeyed(Key, Value);
			}
			else
			{
				TaskArray->Add(Value);
			}
		}
	};

	// Checks the TaskArray is valid using a number of different approaches.
	auto Check = [&]
	{
		if (Head)
		{
			const size_t ExpectedCount = TaskArray->Num();
			size_t Count = 0;

			SECTION("RemoveEachForward")
			{
				FEntry* Current = Head;
				TaskArray->RemoveEachForward([&](int Got)
				{
					REQUIRE(Current);
					REQUIRE(Current->Value == Got);
					Current = Current->Next;
					Count++;
				});
				REQUIRE(Count == ExpectedCount);
				REQUIRE(!Current);
			}

			SECTION("RemoveEachBackward")
			{
				FEntry* Current = Tail;
				TaskArray->RemoveEachBackward([&](int Got)
				{
					REQUIRE(Current);
					REQUIRE(Current->Value == Got);
					Current = Current->Prev;
					Count++;
				});
				REQUIRE(Count == ExpectedCount);
				REQUIRE(!Current);
			}

			SECTION("Reset")
			{
				TaskArray->Reset();
			}
			
			REQUIRE(TaskArray->Num() == 0);
			REQUIRE(TaskArray->IsEmpty());
			REQUIRE(Pool.GetNumInUse() == 0);
		}
		else
		{
			REQUIRE(TaskArray->Num() == 0);
			REQUIRE(TaskArray->IsEmpty());
			TaskArray->RemoveEachForward([&](int)
			{
				FAIL("Should not be called");
			});
			TaskArray->RemoveEachBackward([&](int)
			{
				FAIL("Should not be called");
			});
		}
	};
	
	auto CallWithConfigs = [&](auto&& Fn)
	{
		auto WithKeys = [&](size_t NumKeys)
		{
			SECTION("NumElements: 0")
			{
				Fn(FConfig{NumKeys, /* NumElements */ 0});
			}

			SECTION("NumElements: 10")
			{
				Fn(FConfig{NumKeys, /* NumElements */ 10});
			}

			SECTION("NumElements: 100")
			{
				Fn(FConfig{NumKeys, /* NumElements */ 100});
			}

			SECTION("NumElements: 1000")
			{
				Fn(FConfig{NumKeys, /* NumElements */ 1000});
			}

			SECTION("NumElements: 10000")
			{
				Fn(FConfig{NumKeys, /* NumElements */ 10000});
			}
		};
		
		SECTION("NumKeys: 0")
		{
			WithKeys(/* NumKeys */ 0);
		}

		SECTION("NumKeys: 5")
		{
			WithKeys(/* NumKeys */ 5);
		}

		SECTION("NumKeys: 10")
		{
			WithKeys(/* NumKeys */ 10);
		}
	};

	SECTION("Add")
	{
		CallWithConfigs([&](FConfig Config)
		{
			Populate(Config);
			Check();
		});
	}

	SECTION("DeleteKey")
	{
		CallWithConfigs([&](FConfig Config)
		{
			if (Config.NumKeys == 0)
			{
				return;
			}

			Populate(Config);

			for (size_t I = 0; I < 10; I++)
			{
				void* Key = Keys[Rand() % Config.NumKeys];
				bool HoldsKey = false;
				if (std::vector<FEntry*>& Entries = EntriesByKey[Key]; Entries.size())
				{
					HoldsKey = true;
					DeleteEntry(Entries.back());
					Entries.pop_back();
				}
				REQUIRE(TaskArray->DeleteKey(Key) == HoldsKey);
			}

			Check();
		});
	}

	SECTION("DeleteAllMatchingKeys")
	{
		CallWithConfigs([&](FConfig Config)
		{
			if (Config.NumKeys == 0)
			{
				return;
			}

			Populate(Config);

			for (size_t I = 0; I < 4; I++)
			{
				void* Key = Keys[Rand() % Config.NumKeys];
				std::vector<FEntry*>& Entries = EntriesByKey[Key];
				bool HoldsKey = !Entries.empty();
				while (Entries.size())
				{
					DeleteEntry(Entries.back());
					Entries.pop_back();
				}
				REQUIRE(TaskArray->DeleteAllMatchingKeys(Key) == HoldsKey);
			}

			Check();
		});
	}

	SECTION("AddAll")
	{
		CallWithConfigs([&](FConfig Config)
		{
			Populate(Config);

			auto Second = std::make_unique<TaskArrayType>(Pool);
			std::swap(TaskArray, Second);
			Populate(Config);
			std::swap(TaskArray, Second);

			TaskArray->AddAll(std::move(*Second));

			Check();
		});
	}

	// Clean-up
	while (Head)
	{
		DeleteEntry(Head);
	}
}
