// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "Catch2Includes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Templates/UnrealTemplate.h"
#include <functional>

namespace
{

// Helper for calling Task with the given arguments, which may be a
// AUTORTFM_DISABLED annotated function.
template<typename TASK, typename ... ARGS>
auto CallTask(TASK&& Task, ARGS&& ... Args)
{
	AutoRTFM::UnreachableIfClosed();
	return Task(std::forward<ARGS>(Args)...);
}

} // anonymous namespace

TEST_CASE("TTask")
{
	SECTION("void()")
	{
		using FTask = AutoRTFM::TTask<void()>;

		int ValueA = 0;
		int ValueB = 0;
		int ValueC = 0;
		int ValueD = 0;

		auto SmallLambda = [&ValueA] AUTORTFM_OPEN
		{
			ValueA++;
		};

		auto LargeLambda = [&ValueA, &ValueB, &ValueC, &ValueD] AUTORTFM_OPEN
		{
			ValueA++;
			ValueB++;
			ValueC++;
			ValueD++;
		};

		struct FAlignedStruct
		{
			alignas(16) int& Value;
		};
		
		auto AlignedLambda = [Struct = FAlignedStruct(ValueB)] AUTORTFM_OPEN
		{
			REQUIRE((reinterpret_cast<uintptr_t>(&Struct) & 15) == 0);
			Struct.Value++;
		};

		static_assert(sizeof(SmallLambda) <= FTask::InlineDataSize);
		static_assert(sizeof(LargeLambda) > FTask::InlineDataSize);
		static_assert(sizeof(AlignedLambda) <= FTask::InlineDataSize);
		static_assert(alignof(decltype(AlignedLambda)) > FTask::InlineDataAlignment);

		auto WasSmallLambdaCalled = [&]
		{
			return ValueA == 1 && ValueB == 0 && ValueC == 0 && ValueD == 0;
		};
		auto WasLargeLambdaCalled = [&]
		{
			return ValueA == 1 && ValueB == 1 && ValueC == 1 && ValueD == 1;
		};
		auto WasAlignedLambdaCalled = [&]
		{
			return ValueA == 0 && ValueB == 1 && ValueC == 0 && ValueD == 0;
		};

		SECTION("IsSet")
		{
			REQUIRE(FTask().IsSet() == false);
			REQUIRE(FTask(SmallLambda).IsSet() == true);
			REQUIRE(FTask(LargeLambda).IsSet() == true);
			REQUIRE(FTask(MoveTemp(SmallLambda)).IsSet() == true);
			REQUIRE(FTask(MoveTemp(LargeLambda)).IsSet() == true);
		}

		SECTION("Copy Construct TTask")
		{
			SECTION("Invalid")
			{
				FTask Original;
				FTask Task(Original);
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task(Original);
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task(Original);
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task(Original);
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Copy Construct Lambda")
		{
			SECTION("Small")
			{
				FTask Task(SmallLambda);
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Task(LargeLambda);
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Task(AlignedLambda);
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Copy Construct TFunction")
		{
			TFunction<void()> Function(SmallLambda);
			FTask Task(Function);
			REQUIRE(Function.IsSet() == true);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Copy Construct std::function")
		{
			std::function<void()> Function(SmallLambda);
			FTask Task(Function);
			REQUIRE(static_cast<bool>(Function) == true);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Copy Assign TTask")
		{
			SECTION("Invalid")
			{
				FTask Original;
				FTask Task;
				Task = Original;
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task;
				Task = Original;
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task;
				Task = Original;
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task;
				Task = Original;
				Original.Reset();
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Copy Assign Lambda")
		{
			SECTION("Small")
			{
				FTask Task;
				Task = SmallLambda;
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Task;
				Task = LargeLambda;
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Task;
				Task = AlignedLambda;
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Copy Assign TFunction")
		{
			TFunction<void()> Function(SmallLambda);
			FTask Task;
			Task = Function;
			REQUIRE(Function.IsSet() == true);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Copy Assign std::function")
		{
			std::function<void()> Function(SmallLambda);
			FTask Task;
			Task = Function;
			REQUIRE(static_cast<bool>(Function) == true);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Move Construct Lambda")
		{
			SECTION("Small")
			{
				FTask Task(MoveTemp(SmallLambda));
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Task(MoveTemp(LargeLambda));
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Task(MoveTemp(AlignedLambda));
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Move Construct TTask")
		{
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Move Construct TFunction")
		{
			TFunction<void()> Function(SmallLambda);
			FTask Task(MoveTemp(Function));
			REQUIRE(Function.IsSet() == false);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Move Construct std::function")
		{
			std::function<void()> Function(SmallLambda);
			FTask Task(MoveTemp(Function));
			// Note: standard library implementations of std::function are not
			// guaranteed to invalidate the function on move - hence no check.
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Move Assign Lambda")
		{
			SECTION("Small")
			{
				FTask Task;
				Task = MoveTemp(SmallLambda);
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Task;
				Task = MoveTemp(LargeLambda);
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Task;
				Task = MoveTemp(AlignedLambda);
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Move Assign TTask")
		{
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task;
				Task = MoveTemp(Original);
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasSmallLambdaCalled());
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task;
				Task = MoveTemp(Original);
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasLargeLambdaCalled());
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task;
				Task = MoveTemp(Original);
				REQUIRE(Original.IsSet() == false);
				CallTask(Task);
				REQUIRE(WasAlignedLambdaCalled());
			}
		}

		SECTION("Move Assign TFunction")
		{
			TFunction<void()> Function(SmallLambda);
			FTask Task;
			Task = MoveTemp(Function);
			REQUIRE(Function.IsSet() == false);
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Move Assign std::function")
		{
			std::function<void()> Function(SmallLambda);
			FTask Task;
			Task = MoveTemp(Function);
			// Note: standard library implementations of std::function are not
			// guaranteed to invalidate the function on move - hence no check.
			CallTask(Task);
			REQUIRE(WasSmallLambdaCalled());
		}

		SECTION("Reset")
		{
			SECTION("Invalid")
			{
				FTask Task;
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Small")
			{
				FTask Task(SmallLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Large")
			{
				FTask Task(LargeLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Aligned")
			{
				FTask Task(AlignedLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
		}
	}

	SECTION("int(int, bool)")
	{
		using FTask = AutoRTFM::TTask<int(int, bool)>;

		int ValueA = 0;
		int ValueB = 0;
		int ValueC = 0;
		int ValueD = 0;

		auto SmallLambda = [&ValueA](int I, bool B)
		{
			ValueA = B ? I : -I;
			return I;
		};

		auto LargeLambda = [&ValueA, &ValueB, &ValueC, &ValueD](int I, bool B)
		{
			ValueA = B ? I : -I;
			ValueB = B ? -I : I;
			ValueC = B ? I : -I;
			ValueD = B ? -I : I;
			return I;
		};

		struct FAlignedStruct
		{
			alignas(16) int& Value;
		};
		
		auto AlignedLambda = [Struct = FAlignedStruct(ValueB)](int I, bool B)
		{
			REQUIRE((reinterpret_cast<uintptr_t>(&Struct) & 15) == 0);
			Struct.Value = B ? I : -I;
			return I;
		};

		static_assert(sizeof(SmallLambda) <= FTask::InlineDataSize);
		static_assert(sizeof(LargeLambda) > FTask::InlineDataSize);
		static_assert(sizeof(AlignedLambda) <= FTask::InlineDataSize);
		static_assert(alignof(decltype(AlignedLambda)) > FTask::InlineDataAlignment);

		auto WasSmallLambdaCalled = [&](int Expected)
		{
			return ValueA == Expected && ValueB == 0 && ValueC == 0 && ValueD == 0;
		};
		auto WasLargeLambdaCalled = [&](int Expected)
		{
			return ValueA == Expected && ValueB == -Expected && ValueC == Expected && ValueD == -Expected;
		};
		auto WasAlignedLambdaCalled = [&](int Expected)
		{
			return ValueA == 0 && ValueB == Expected && ValueC == 0 && ValueD == 0;
		};

		SECTION("IsValid")
		{
			REQUIRE(FTask().IsSet() == false);
			REQUIRE(FTask(SmallLambda).IsSet() == true);
			REQUIRE(FTask(LargeLambda).IsSet() == true);
			REQUIRE(FTask(MoveTemp(SmallLambda)).IsSet() == true);
			REQUIRE(FTask(MoveTemp(LargeLambda)).IsSet() == true);
		}

		SECTION("Copy Construct Lambda")
		{
			SECTION("Small")
			{
				FTask Task(SmallLambda);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasSmallLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasSmallLambdaCalled(-20));
			}
			SECTION("Large")
			{
				FTask Task(LargeLambda);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasLargeLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasLargeLambdaCalled(-20));
			}
			SECTION("Aligned")
			{
				FTask Task(AlignedLambda);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasAlignedLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasAlignedLambdaCalled(-20));
			}
		}

		SECTION("Copy Construct TTask")
		{
			SECTION("Invalid")
			{
				FTask Original;
				FTask Task(Original);
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task(Original);
				Original.Reset();
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasSmallLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasSmallLambdaCalled(-20));
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task(Original);
				Original.Reset();
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasLargeLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasLargeLambdaCalled(-20));
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task(Original);
				Original.Reset();
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasAlignedLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasAlignedLambdaCalled(-20));
			}
		}

		SECTION("Move Construct Lambda")
		{
			SECTION("Small")
			{
				FTask Task(MoveTemp(SmallLambda));
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasSmallLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasSmallLambdaCalled(-20));
			}
			SECTION("Large")
			{
				FTask Task(MoveTemp(LargeLambda));
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasLargeLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasLargeLambdaCalled(-20));
			}
			SECTION("Aligned")
			{
				FTask Task(MoveTemp(AlignedLambda));
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasAlignedLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasAlignedLambdaCalled(-20));
			}
		}

		SECTION("Move Construct TTask")
		{
			SECTION("Small")
			{
				FTask Original(SmallLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasSmallLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasSmallLambdaCalled(-20));
			}
			SECTION("Large")
			{
				FTask Original(LargeLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasLargeLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasLargeLambdaCalled(-20));
			}
			SECTION("Aligned")
			{
				FTask Original(AlignedLambda);
				FTask Task(MoveTemp(Original));
				REQUIRE(Original.IsSet() == false);
				REQUIRE(CallTask(Task, 10, true) == 10);
				REQUIRE(WasAlignedLambdaCalled(10));
				REQUIRE(CallTask(Task, 20, false) == 20);
				REQUIRE(WasAlignedLambdaCalled(-20));
			}
		}

		SECTION("Reset")
		{
			SECTION("Invalid")
			{
				FTask Task;
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Small")
			{
				FTask Task(SmallLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Large")
			{
				FTask Task(LargeLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
			SECTION("Aligned")
			{
				FTask Task(AlignedLambda);
				Task.Reset();
				REQUIRE(Task.IsSet() == false);
			}
		}
	}
}
