// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"
#include "Logging/LogMacros.h"

#include <atomic>
#include <cmath>
#include <cstdio>
#include <cstring>
#include <stdio.h>
#include <thread>
#include <vector>

#if PLATFORM_CPU_X86_FAMILY
#include <immintrin.h>
#endif // PLATFORM_CPU_X86_FAMILY

DECLARE_LOG_CATEGORY_EXTERN(LogAutoRTFMTests, Display, All)
DEFINE_LOG_CATEGORY(LogAutoRTFMTests)

TEST_CASE("Tests.WriteInt")
{
    int X = 1;
    REQUIRE(
        AutoRTFM::ETransactionResult::Committed ==
        AutoRTFM::Transact([&] () { X = 2; }));
    REQUIRE(X == 2);
}

TEST_CASE("Tests.UE_LOG")
{
	AutoRTFM::Testing::Commit([&]
	{
		UE_LOGF(LogAutoRTFMTests, Display, "Testing this works!");
	});
}

// This test ensures that if you have STM and non-STM modifying data that is
// adjacent in memory, the STM code won't lose modifications to data that
// happens to fall into the same STM line.
TEST_CASE("stm.no_trashing_non_stm", "[.multi-threaded-test]")
{
	// A hit-count - lets us ensure each thread is launched and running before
	// we kick off the meat of the test.
	std::atomic_uint HitCount(0);

	// We need a data per thread to ensure this test works! We heap allocate
	// this in a std::vector because we get a 'free' alignment of the buffer,
	// rather than a potential 4-byte alignment on the stack which could cause
	// the data to go into different lines in the STM implementation.
	// TODO: use memalign explicitly here?
	std::vector<unsigned int> Datas(2);

	auto non_stm = std::thread([&HitCount, &Datas] AUTORTFM_OPEN (unsigned int index)
	{
		const auto Load = Datas[index];

		// Increment the hit count to unlock the STM thread.
		HitCount++;

		// Wait for the STM thread to signal that it has Loaded.
		while (HitCount != 2) {}

		// Then do our store which the STM was prone to losing.
		Datas[index] = Load + 1;

		// And lastly unlock the STM one last time.
		HitCount++;
	}, 0);

	auto stmified = std::thread([&HitCount, &Datas] AUTORTFM_OPEN (unsigned int index)
	{
		// Wait for the non-STM thread to have Loaded data.
		while (HitCount != 1) {}

		auto transaction = AutoRTFM::Transact([&] ()
		{
			const auto Load = Datas[index];

			// Now do a naughty open so that we can fiddle with the atomic and
			// the non-STM thread can see that immediately.
			AutoRTFM::Open([&] ()
			{
				// Unblock the non-STM thread and let it do its store.
				HitCount++;

				// Wait for the non-STM thread to signal that it has done its
				// store.
				while(HitCount != 3) {}
			});

			// Then do our store which the STM was prone to losing.
			Datas[index] = Load + 1;
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == transaction);
	}, 1);

	non_stm.join();
	stmified.join();

	REQUIRE(Datas[0] == 1);
	REQUIRE(Datas[1] == 1);
}

// A test case that ensures that read invalidation works as intended.
TEST_CASE("stm.read_invalidation_works", "[.multi-threaded-test]")
{
	// A hit-count - lets us ensure each thread is launched and running before
	// we kick off the meat of the test.
	std::atomic_uint HitCount(0);

	// We need a data per thread to ensure this test works! We heap allocate
	// this in a std::vector because we get a 'free' alignment of the buffer,
	// rather than a potential 4-byte alignment on the stack which could cause
	// the data to go into different lines in the STM implementation.
	// TODO: use memalign explicitly here?
	std::vector<unsigned int> Datas(3);

	auto stm_write_only = std::thread([&] AUTORTFM_OPEN
	{
		auto transaction = AutoRTFM::Transact([&] ()
		{
			// Do a non-transactional open to allow us to order the execution
			// pattern between two competing transactions.
			AutoRTFM::Open([&] ()
			{
				// Wait for the read-write thread.
				while(HitCount != 1) {}
			});

			Datas[0] = 42;
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == transaction);

		// Now that our transaction is complete, unblock the read-write thread.
		HitCount++;
	});

	auto stm_read_write = std::thread([&] AUTORTFM_OPEN
	{
		auto transaction = AutoRTFM::Transact([&] ()
		{
			// Read the data that the write-only thread will be writing to.
			const auto Load = Datas[0];

			AutoRTFM::Open([&] ()
			{
				// Tell the write-only thread to continue.
				HitCount++;

				// Wait for the write-only thread.
				for(;;)
				{
					if (2 <= HitCount)
					{
						// This store simulates when a non-STM thread would
						// be modifying data adjacent to our STM data.
						Datas[2]++;
						break;
					}
				}
			});

			// Then do a store - this store will cause the transaction to fail.
			Datas[1] = Load + 1;
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == transaction);
	});

	stm_write_only.join();
	stm_read_write.join();

	REQUIRE(Datas[0] == 42);
	REQUIRE(Datas[1] == 43);

	// 2 because we fail the transaction the first time, and commit the second.
	REQUIRE(Datas[2] == 2);
}

TEST_CASE("stm.memcpy")
{
	constexpr unsigned Size = 1024;

	unsigned char Reference[Size];

	for (unsigned i = 0; i < Size; i++)
	{
		Reference[i] = i % UINT8_MAX;
	}

	std::unique_ptr<unsigned char[]> Datas(nullptr);

	auto Transaction = AutoRTFM::Transact([&]()
	{
		Datas.reset(new unsigned char[Size]);

		memcpy(Datas.get(), Reference, Size);
	});

	REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);

	for (unsigned i = 0; i < Size; i++)
	{
		REQUIRE((unsigned)Reference[i] == (unsigned)Datas[i]);
	}
}

TEST_CASE("stm.memmove")
{
	SECTION("lower")
	{
		constexpr unsigned Window = 1024;
		constexpr unsigned Size = Window + 2;

		unsigned char Datas[Size];

		for (unsigned i = 0; i < Size; i++)
		{
			Datas[i] = i % UINT8_MAX;
		}

		auto Transaction = AutoRTFM::Transact([&]()
		{
			memmove(Datas + 1, Datas, Window);
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);

		REQUIRE(0 == (unsigned)Datas[0]);

		for (unsigned i = 0; i < Window; i++)
		{
			REQUIRE((i % UINT8_MAX) == (unsigned)Datas[i + 1]);
		}

		REQUIRE(((Size - 1) % UINT8_MAX) == (unsigned)Datas[Size - 1]);
	}

	SECTION("higher")
	{
		constexpr unsigned Window = 1024;
		constexpr unsigned Size = Window + 2;

		unsigned char Datas[Size];

		for (unsigned i = 0; i < Size; i++)
		{
			Datas[i] = i % UINT8_MAX;
		}

		auto Transaction = AutoRTFM::Transact([&]()
		{
			memmove(Datas, Datas + 1, Window);
		});

		REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);

		for (unsigned i = 0; i < Window; i++)
		{
			REQUIRE(((i + 1) % UINT8_MAX) == (unsigned)Datas[i]);
		}

		REQUIRE(((Size - 2) % UINT8_MAX) == (unsigned)Datas[Size - 2]);
		REQUIRE(((Size - 1) % UINT8_MAX) == (unsigned)Datas[Size - 1]);
	}
}

TEST_CASE("stm.memset")
{
	constexpr unsigned Size = 1024;

	unsigned char Datas[Size];

	for (unsigned i = 0; i < Size; i++)
	{
		Datas[i] = i % UINT8_MAX;
	}

	auto Transaction = AutoRTFM::Transact([&]()
	{
		memset(Datas, 42, Size);
	});

	REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);

	for (unsigned i = 0; i < Size; i++)
	{
		REQUIRE(42 == (unsigned)Datas[i]);
	}
}

TEST_CASE("libc.isnan(float)")
{
	float X = 0.0f;
	float Y = NAN;
	bool bXIsNaN = true;
	bool bYIsNaN = false;
	auto Transaction = AutoRTFM::Transact([&]()
	{
		bXIsNaN = std::isnan(X);
		bYIsNaN = std::isnan(Y);
	});
	REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);
	REQUIRE(false == bXIsNaN);
	REQUIRE(true == bYIsNaN);
}

TEST_CASE("libc.isnan(double)")
{
	double X = 0.0;
	double Y = NAN;
	bool bXIsNaN = true;
	bool bYIsNaN = false;
	auto Transaction = AutoRTFM::Transact([&]()
	{
		bXIsNaN = std::isnan(X);
		bYIsNaN = std::isnan(Y);
	});
	REQUIRE(AutoRTFM::ETransactionResult::Committed == Transaction);
	REQUIRE(false == bXIsNaN);
	REQUIRE(true == bYIsNaN);
}

TEST_CASE("Tests.RetryNonNested")
{
	// We only run this test if we are retrying non-nested transactions (it proves we retried!).
	if (!AutoRTFM::ForTheRuntime::ShouldRetryNonNestedTransactions())
	{
		return;
	}

	unsigned Count = 0;

	AutoRTFM::Testing::Commit([&]
	{
		AutoRTFM::Open([&]
		{
			Count++;
		});
	});

	REQUIRE(2 == Count);
}

TEST_CASE("Tests.fflush")
{
	auto Result = AutoRTFM::Transact([&]()
	{
		// There isn't a simple way to verify that fflush has actually done anything,
		// but we want to at least verify that it can be called safely.
		fflush(stdout);
		std::fflush(stdout);
	});
	REQUIRE(Result == AutoRTFM::ETransactionResult::Committed);
}

#if PLATFORM_CPU_X86_FAMILY
template<int WhichMaskOff>
void AVXDoMaskedStore(double* Vector)
{
	__m256i Mask = _mm256_setr_epi64x(0xFFFFFFFFFFFFFFFFull, 0xFFFFFFFFFFFFFFFFull, 0xFFFFFFFFFFFFFFFFull, 0xFFFFFFFFFFFFFFFFull);
	Mask = _mm256_insert_epi64(Mask, 0x0000000000000000ull, WhichMaskOff);
	__m256d Val = _mm256_setr_pd(101.0, 102.0, 103.0, 104.0);
	_mm256_maskstore_pd(Vector, Mask, Val);
}

TEST_CASE("Tests.AVXMaskedStore")
{
	auto RequireVectorsEqual = [] AUTORTFM_OPEN (double* VectorPtr, __m256d Rhs)
	{
		__m256d Lhs = _mm256_loadu_pd(VectorPtr);
		__m256 CmpResult = _mm256_castpd_ps(_mm256_cmp_pd(Lhs, Rhs, 0));
		int Result = _mm256_movemask_ps(CmpResult);
		REQUIRE(Result == 0xFF);
	};

	{
		double Vector[4] = {1.0, 2.0, 3.0, 4.0};
		auto TransactResult = AutoRTFM::Transact([&]()
		{
			// do a masked store to Vector
			AVXDoMaskedStore<0>(&Vector[0]);

			RequireVectorsEqual(&Vector[0], _mm256_setr_pd(1.0, 102.0, 103.0, 104.0));

			// we overwrite the non-written value in the Open before we abort, to ensure
			// that the runtime only rolls back elements that the masked write wrote to
			AutoRTFM::Open([&]() { Vector[0] = 99.0; });

    		AutoRTFM::AbortTransaction();
		});

		REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
		RequireVectorsEqual(&Vector[0], _mm256_setr_pd(99.0, 2.0, 3.0, 4.0));
	}

	{
		double Vector[4] = {1.0, 2.0, 3.0, 4.0};
		auto TransactResult = AutoRTFM::Transact([&]()
		{
			// do a masked store to Vector
			AVXDoMaskedStore<1>(&Vector[0]);

			RequireVectorsEqual(&Vector[0], _mm256_setr_pd(101.0, 2.0, 103.0, 104.0));

			// we overwrite the non-written value in the Open before we abort, to ensure
			// that the runtime only rolls back elements that the masked write wrote to
			AutoRTFM::Open([&]() { Vector[1] = 99.0; });

    		AutoRTFM::AbortTransaction();
		});

		REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
		RequireVectorsEqual(&Vector[0], _mm256_setr_pd(1.0, 99.0, 3.0, 4.0));
	}

	{
		double Vector[4] = {1.0, 2.0, 3.0, 4.0};
		auto TransactResult = AutoRTFM::Transact([&]()
		{
			// do a masked store to Vector
			AVXDoMaskedStore<2>(&Vector[0]);

			RequireVectorsEqual(&Vector[0], _mm256_setr_pd(101.0, 102.0, 3.0, 104.0));

			// we overwrite the non-written value in the Open before we abort, to ensure
			// that the runtime only rolls back elements that the masked write wrote to
			AutoRTFM::Open([&]() { Vector[2] = 99.0; });

    		AutoRTFM::AbortTransaction();
		});

		REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
		RequireVectorsEqual(&Vector[0], _mm256_setr_pd(1.0, 2.0, 99.0, 4.0));
	}

	{
		double Vector[4] = {1.0, 2.0, 3.0, 4.0};
		auto TransactResult = AutoRTFM::Transact([&]()
		{
			// do a masked store to Vector
			AVXDoMaskedStore<3>(&Vector[0]);

			RequireVectorsEqual(&Vector[0], _mm256_setr_pd(101.0, 102.0, 103.0, 4.0));

			// we overwrite the non-written value in the Open before we abort, to ensure
			// that the runtime only rolls back elements that the masked write wrote to
			AutoRTFM::Open([&]() { Vector[3] = 99.0; });

    		AutoRTFM::AbortTransaction();
		});

		REQUIRE(TransactResult == AutoRTFM::ETransactionResult::AbortedByRequest);
		RequireVectorsEqual(&Vector[0], _mm256_setr_pd(1.0, 2.0, 3.0, 99.0));
	}
}
#endif // PLATFORM_CPU_X86_FAMILY

#if PLATFORM_WINDOWS

TEST_CASE("Tests.__local_stdio_printf_options")
{
	unsigned __int64* NonTransactional = __local_stdio_printf_options();
	unsigned __int64* Transactional = nullptr;

	AutoRTFM::Testing::Commit([&]
		{
			Transactional = __local_stdio_printf_options();
		});

	REQUIRE(NonTransactional == Transactional);
}

TEST_CASE("Tests.__local_stdio_scanf_options")
{
	unsigned __int64* NonTransactional = __local_stdio_scanf_options();
	unsigned __int64* Transactional = nullptr;

	AutoRTFM::Testing::Commit([&]
		{
			Transactional = __local_stdio_scanf_options();
		});

	REQUIRE(NonTransactional == Transactional);
}

#endif // PLATFORM_WINDOWS

TEST_CASE("Tests.thread_local")
{
	auto TLSInt = [] AUTORTFM_ENABLE () -> int&
	{
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(int, MyInt);
		return MyInt;
	};
	auto TLSString = [] AUTORTFM_ENABLE () -> FString&
	{
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FString, MyString);
		return MyString;
	};

	SECTION("Abort on Initial Access")
	{
		AutoRTFM::Testing::Abort([&]
		{
			TLSInt() = 42;
			TLSString() = "Cat";
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(TLSInt() == 0);
		REQUIRE(TLSString() == "");
	}

	SECTION("Abort")
	{
		TLSInt() = 123;
		TLSString() = "Pickle";
		AutoRTFM::Testing::Abort([&]
		{
			TLSInt() = 456;
			TLSString() = "Peanut";
			AutoRTFM::AbortTransaction();
		});
		REQUIRE(TLSInt() == 123);
		REQUIRE(TLSString() == "Pickle");
	}

	SECTION("Commit")
	{
		AutoRTFM::Testing::Commit([&]
		{
			TLSInt() = 42;
			TLSString() = "Cat";
		});
		REQUIRE(TLSInt() == 42);
		REQUIRE(TLSString() == "Cat");
	}
}
