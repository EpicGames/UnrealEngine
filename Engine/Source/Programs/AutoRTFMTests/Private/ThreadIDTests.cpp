// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThreadID.h"
#include "Catch2Includes.h"

#include <condition_variable>
#include <cstddef>
#include <mutex>
#include <thread>
#include <vector>
#include <unordered_set>

TEST_CASE("ThreadID")
{
	REQUIRE(AutoRTFM::FThreadID{} == AutoRTFM::FThreadID::Invalid);
	
	// Synchronisation primitive used to ensure all threads have obtained
	// their ID before continuing on the main thread.
	class AUTORTFM_OPEN FWaitGroup
	{
	public:
		// Constructor. Creates the WaitGroup with an the initial count.
		FWaitGroup(size_t Count) : Count{Count}{}

		// Blocks until Done() is called Count times.
		void Wait()
		{
			std::unique_lock<std::mutex> Lock{Mutex};
			ConditionVariable.wait(Lock, [this] { return Count == 0; });
		}

		// Decrements Count, unblocking any Wait() calls when Count reaches 0.
		void Done()
		{
			std::unique_lock<std::mutex> Lock{Mutex};
			Count--;
			if (Count == 0)
			{
				ConditionVariable.notify_all();
			}
		}

	private:
		std::mutex Mutex;
		std::condition_variable ConditionVariable;
		size_t Count = 0;
	};

	// A STL hasher for AutoRTFM::FThreadID so we can use them in a std::unordered_map.
	struct FThreadIDHasher
	{
		size_t operator()(const AutoRTFM::FThreadID& ThreadID) const
		{
			return std::hash<decltype(ThreadID.Value)>{}(ThreadID.Value);
		}
	};

	const size_t NumThreads = 10;
	std::vector<std::thread> Threads(NumThreads);
	std::vector<AutoRTFM::FThreadID> ThreadIDs(NumThreads);
	FWaitGroup WaitGroup{NumThreads};
	for (size_t I = 0; I < 10; I++)
	{
		Threads[I] = std::thread([I, &WaitGroup, &ThreadIDs] AUTORTFM_OPEN
		{
			ThreadIDs[I] = AutoRTFM::FThreadID::GetCurrent();
			WaitGroup.Done();
		});
	}

	// Wait for all the threads to populate ThreadIDs
	WaitGroup.Wait();

	// Check that all the thread identifiers are unique
	std::unordered_set<AutoRTFM::FThreadID, FThreadIDHasher> Set;
	Set.emplace(AutoRTFM::FThreadID::GetCurrent());
	for (AutoRTFM::FThreadID ThreadID : ThreadIDs)
	{
		bool bUnique = Set.emplace(ThreadID).second;
		REQUIRE(bUnique);
	}
	
	// Join the threads
	for (std::thread& Thread : Threads)
	{
		Thread.join();
	}
}
