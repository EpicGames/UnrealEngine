// Copyright Epic Games, Inc. All Rights Reserved.

// These tests rely on us reaching into core using private
// header includes which is sometimes not so fine.
#if !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "Internationalization/TextCache.h"
#include "Internationalization/TextHistory.h"

DEFINE_LOG_CATEGORY_STATIC(LogAutoRTFM_UECoreTests, Log, All)

TEST_CASE("UECore.FTextHistory")
{
	struct AUTORTFM_ENABLE MyTextHistory final : FTextHistory_Base
	{
		// Need this to always return true so we hit the fun transactional bits!
		bool CanUpdateDisplayString() override
		{
			return true;
		}

		MyTextHistory(const FTextId& InTextId, FString&& InSourceString) : FTextHistory_Base(InTextId, MoveTemp(InSourceString)) {}
	};

	FTextKey Namespace("NAMESPACE");
	FTextKey Key("KEY");
	FTextId TextId(Namespace, Key);
	FString String("WOWWEE");

	MyTextHistory History(TextId, MoveTemp(String));

	SECTION("With Abort")
	{
		AutoRTFM::Testing::Abort([&]()
		{
			History.UpdateDisplayStringIfOutOfDate();
			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("With Commit")
	{
		AutoRTFM::Testing::Commit([&]()
		{
			History.UpdateDisplayStringIfOutOfDate();
		});
	}
}

TEST_CASE("UECore.FTextCache")
{
	// FTextCache is a singleton. Grab its reference.
	FTextCache& Cache = FTextCache::Get();

	// Use a fixed cache key for the tests below.
	const FTextId Key{ TEXT("NAMESPACE"), TEXT("KEY") };

	// As FTextCache does not supply any way to query what's held in the cache,
	// the best we can do here is to call FindOrCache() and check the returned
	// FText strings are as expected.
	auto CheckCacheHealthy = [&]
		{
			FText LookupA = Cache.FindOrCache(TEXT("VALUE"), Key);
			REQUIRE(LookupA.ToString() == TEXT("VALUE"));
			FText LookupB = Cache.FindOrCache(TEXT("REPLACEMENT"), Key);
			REQUIRE(LookupB.ToString() == TEXT("REPLACEMENT"));
			Cache.RemoveCache(Key);
		};

	SECTION("FindOrCache() Add new")
	{
		SECTION("With Abort")
		{
			AutoRTFM::Testing::Abort([&]()
				{
					Cache.FindOrCache(TEXT("VALUE"), Key);
					AutoRTFM::AbortTransaction();
				});

			CheckCacheHealthy();
		}

		SECTION("With Commit")
		{
			AutoRTFM::Testing::Commit([&]()
				{
					Cache.FindOrCache(TEXT("VALUE"), Key);
				});

			CheckCacheHealthy();
		}
	}

	SECTION("FindOrCache() Replace with same value")
	{
		SECTION("With Abort")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("VALUE"), Key);

			AutoRTFM::Testing::Abort([&]()
				{
					Cache.FindOrCache(TEXT("REPLACEMENT"), Key);
					AutoRTFM::AbortTransaction();
				});

			CheckCacheHealthy();
		}

		SECTION("With Commit")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("VALUE"), Key);

			AutoRTFM::Testing::Commit([&]()
				{
					Cache.FindOrCache(TEXT("VALUE"), Key);
				});

			CheckCacheHealthy();
		}
	}

	SECTION("FindOrCache() Replace with different value")
	{
		SECTION("With Abort")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("ORIGINAL"), Key);

			AutoRTFM::Testing::Abort([&]()
				{
					Cache.FindOrCache(TEXT("REPLACEMENT"), Key);
					AutoRTFM::AbortTransaction();
				});

			CheckCacheHealthy();
		}

		SECTION("With Commit")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("ORIGINAL"), Key);

			AutoRTFM::Testing::Commit([&]()
				{
					Cache.FindOrCache(TEXT("REPLACEMENT"), Key);
				});

			CheckCacheHealthy();
		}
	}

	static constexpr bool bSupportsTransactionalRemoveCache = false; // #jira SOL-6743
	if (!bSupportsTransactionalRemoveCache)
	{
		return;
	}

	SECTION("RemoveCache()")
	{
		SECTION("With Abort")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("VALUE"), Key);

			AutoRTFM::Testing::Abort([&]()
				{
					Cache.RemoveCache(Key);
					AutoRTFM::AbortTransaction();
				});

			CheckCacheHealthy();
		}

		SECTION("With Commit")
		{
			// Add an entry to the cache before the transaction
			Cache.FindOrCache(TEXT("VALUE"), Key);

			AutoRTFM::Testing::Commit([&]()
				{
					Cache.RemoveCache(Key);
				});

			CheckCacheHealthy();
		}
	}


	SECTION("Mixed Closed & Open")
	{
		SECTION("Closed: FindOrCache() Open: RemoveCache()")
		{
			SECTION("With Abort")
			{
				AutoRTFM::Testing::Abort([&]()
					{
						Cache.FindOrCache(TEXT("VALUE"), Key);
						AutoRTFM::Open([&] { Cache.RemoveCache(Key); });
						AutoRTFM::AbortTransaction();
					});

				CheckCacheHealthy();
			}

			SECTION("With Commit")
			{
				AutoRTFM::Testing::Commit([&]()
					{
						Cache.FindOrCache(TEXT("VALUE"), Key);
						AutoRTFM::Open([&] { Cache.RemoveCache(Key); });
					});

				CheckCacheHealthy();
			}
		}
	}
}

#endif // !defined(AUTORTFMTESTS_NO_PEEKING_AT_PRIVATES)
