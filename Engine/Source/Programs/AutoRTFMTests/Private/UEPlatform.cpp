// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "AutoRTFM.h"
#include "AutoRTFMTestUtils.h"
#include "Catch2Includes.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Guid.h"


TEST_CASE("FPlatformMisc.CreateGUID")
{
	FGuid Guid;

	SECTION("Commit")
	{
		SECTION("GuidOutsideTransactionStack")
		{
			AutoRTFM::Testing::Commit([&]
			{
				FPlatformMisc::CreateGuid(Guid);
			});
		}

		SECTION("GuidInsideTransactionStack")
		{
			AutoRTFM::Testing::Commit([&]
			{
				FGuid InnerGuid;
				FPlatformMisc::CreateGuid(InnerGuid);
				AutoRTFM::Open([&]{ Guid = InnerGuid; });
			});
		}

		REQUIRE(Guid != FGuid());
	}

	SECTION("Abort")
	{
		SECTION("GuidOutsideTransactionStack")
		{
			AutoRTFM::Testing::Abort([&]
			{
				FPlatformMisc::CreateGuid(Guid);
				AutoRTFM::AbortTransaction();
			});
		}

		SECTION("GuidInsideTransactionStack")
		{
			AutoRTFM::Testing::Abort([&]
			{
				FGuid InnerGuid;
				FPlatformMisc::CreateGuid(InnerGuid);
				Guid = InnerGuid;
				AutoRTFM::AbortTransaction();
			});
		}

		REQUIRE(Guid == FGuid());
	}
}

TEST_CASE("FGuid.ImportTextItem")
{
	TCHAR Buffer[64] = TEXT("f06250a3d866649e3b3d77f936fe6620");
	const TCHAR* GuidString = Buffer;
	FGuid Expected;
	REQUIRE(FGuid::Parse(GuidString, Expected));
	
	FGuid Guid;

	SECTION("Commit")
	{
		AutoRTFM::Testing::Commit([&]
		{
			bool Result = Guid.ImportTextItem(GuidString, 0, nullptr, nullptr);
			REQUIRE(Result == true);
		});

		REQUIRE((GuidString - Buffer) == 32);
		REQUIRE(Guid == Expected);
	}

	SECTION("Abort")
	{
		AutoRTFM::Testing::Abort([&]
		{
			bool Result = Guid.ImportTextItem(GuidString, 0, nullptr, nullptr);
			REQUIRE(Result == true);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(GuidString == Buffer);
		REQUIRE(Guid == FGuid());
	}
}
