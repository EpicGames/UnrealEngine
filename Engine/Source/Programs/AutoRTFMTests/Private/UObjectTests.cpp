// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "ScopedGuard.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/MallocLeakDetection.h"
#include "MyAutoRTFMTestObject.h"
#include "MyAutoRTFMTestObjectWithSubObjects.h"
#include "UObject/GCObject.h"
#include "UObject/ReachabilityAnalysis.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectThreadContext.h"

#include "Catch2Includes.h"

TEST_CASE("UObject.NewObject")
{
	SECTION("Create")
	{
		UMyAutoRTFMTestObject* Object = nullptr;

		AutoRTFM::Testing::Commit([&]
		{
			Object = NewObject<UMyAutoRTFMTestObject>();
		});

		REQUIRE(nullptr != Object);
		REQUIRE(42 == Object->Value);
	}

	SECTION("Abort")
	{
		UMyAutoRTFMTestObject* Object = nullptr;

		AutoRTFM::Testing::Abort([&]
		{
			Object = NewObject<UMyAutoRTFMTestObject>();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(nullptr == Object);
	}
}

TEST_CASE("UObject.NewObjectWithOuter")
{
	SECTION("Create")
	{
		UMyAutoRTFMTestObject* Outer = NewObject<UMyAutoRTFMTestObject>();
		UMyAutoRTFMTestObject* Object = nullptr;

		AutoRTFM::Testing::Commit([&]
		{
			Object = NewObject<UMyAutoRTFMTestObject>(Outer);
		});

		REQUIRE(nullptr != Object);
		REQUIRE(42 == Object->Value);
		REQUIRE(Object->IsInOuter(Outer));
		REQUIRE(55 == Outer->Value);
	}

	SECTION("Abort")
	{
		UMyAutoRTFMTestObject* Outer = NewObject<UMyAutoRTFMTestObject>();
		UMyAutoRTFMTestObject* Object = nullptr;

		AutoRTFM::Testing::Abort([&]
		{
			Object = NewObject<UMyAutoRTFMTestObject>(Outer);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(nullptr == Object);
		REQUIRE(42 == Outer->Value);
	}
}

TEST_CASE("UObject.Rename")
{
	const TCHAR* Cat = TEXT("Cat");
	const TCHAR* Dog = TEXT("Dog");
	const TCHAR* Bat = TEXT("Bat");
	UMyAutoRTFMTestObject* OuterA = NewObject<UMyAutoRTFMTestObject>();
	UMyAutoRTFMTestObject* OuterB = NewObject<UMyAutoRTFMTestObject>();
	UMyAutoRTFMTestObject* OuterC = NewObject<UMyAutoRTFMTestObject>();
	UMyAutoRTFMTestObject* Object = NewObject<UMyAutoRTFMTestObject>(OuterA, Cat);
	REQUIRE(Object->GetOuter() == OuterA);

	SECTION("Commit(Rename(Name))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
		});

		REQUIRE(Object->GetName() == Dog);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Abort(Rename(Name))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Commit(Rename(Name), Rename(Name))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
			REQUIRE(Object->Rename(Bat, OuterA));
		});

		REQUIRE(Object->GetName() == Bat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Abort(Rename(Name), Rename(Name))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
			REQUIRE(Object->Rename(Bat, OuterA));
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Commit(Rename(Name), Commit(Rename(Name)))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Object->Rename(Bat, OuterA));
			});
		});

		REQUIRE(Object->GetName() == Bat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Commit(Rename(Name), Abort(Rename(Name)))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(Dog, OuterA));
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(Object->Rename(Bat, OuterA));
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(Object->GetName() == Dog);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Abort(Rename(Name), Commit(Rename(Name)))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(Bat, OuterA));
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Object->Rename(Dog, OuterA));
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Abort(Rename(Name), Commit(Rename(Name)))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(Bat, OuterA));
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(Object->Rename(Dog, OuterA));
				AutoRTFM::AbortTransaction();
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Commit(Rename(Object))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterB);
	}

	SECTION("Abort(Rename(Object))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}
	
	SECTION("Commit(Rename(Object), Rename(Object))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			REQUIRE(Object->Rename(nullptr, OuterC));
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterC);
	}

	SECTION("Abort(Rename(Object), Rename(Object))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			REQUIRE(Object->Rename(nullptr, OuterC));
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Commit(Rename(Object), Commit(Rename(Object)))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Object->Rename(nullptr, OuterC));
			});
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterC);
	}

	SECTION("Commit(Rename(Object), Abort(Rename(Object)))")
	{
		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(Object->Rename(nullptr, OuterC));
				AutoRTFM::AbortTransaction();
			});
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterB);
	}

	SECTION("Abort(Rename(Object), Commit(Rename(Object)))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(Object->Rename(nullptr, OuterC));
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}

	SECTION("Abort(Rename(Object), Commit(Rename(Object)))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(Object->Rename(nullptr, OuterB));
			AutoRTFM::Testing::Abort([&]
			{
				REQUIRE(Object->Rename(nullptr, OuterC));
				AutoRTFM::AbortTransaction();
			});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->GetName() == Cat);
		REQUIRE(Object->GetOuter() == OuterA);
	}
}

// This is a copy of the helper function in TestGarbageCollector.cpp.
int32 PerformGarbageCollectionWithIncrementalReachabilityAnalysis(TFunctionRef<bool(int32)> ReachabilityIterationCallback)
{
	int32 ReachabilityIterationIndex = 0;

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, false);

	while (IsIncrementalReachabilityAnalysisPending())
	{
		if (ReachabilityIterationCallback(ReachabilityIterationIndex))
		{
			break;
		}

		// Re-check if incremental rachability is still pending because the callback above could've triggered GC which would complete all iterations
		if (IsIncrementalReachabilityAnalysisPending())
		{
			PerformIncrementalReachabilityAnalysis(GetReachabilityAnalysisTimeLimit());
			ReachabilityIterationIndex++;
		}
	}

	if (IsIncrementalPurgePending())
	{
		IncrementalPurgeGarbage(false);
	}
	check(IsIncrementalPurgePending() == false);

	return ReachabilityIterationIndex + 1;
}

TEST_CASE("UObject.MarkAsReachable")
{
	// We need incremental reachability to be on.
	SetIncrementalReachabilityAnalysisEnabled(true);

	// Cache the original time limit.
	const float Original = GetReachabilityAnalysisTimeLimit();

	// And we need a super small time limit s that reachability analysis will definitely have started.
	SetReachabilityAnalysisTimeLimit(FLT_MIN);
	
	// We need to be sure we've done the static GC initialization before we start doing a garbage
	// collection.
	FGCObject::StaticInit();

	UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();

	// Somewhat ironically, garbage collection can leak memory.
	MALLOCLEAK_IGNORE_SCOPE();

	PerformGarbageCollectionWithIncrementalReachabilityAnalysis([Object](int32 index)
	{
		if (0 != index)
		{
			return true;
		}

		AutoRTFM::Testing::Commit([&]
		{
			Object->MarkAsReachable();
		});

		return false;
	});

	// Reset it back just incase another test required the original time limit.
	SetReachabilityAnalysisTimeLimit(Original);
}

TEST_CASE("UObject")
{
	SECTION("MarkAsGarbage")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();

		AutoRTFM::Testing::Commit([&]
		{
			Object->MarkAsGarbage();
		});

		REQUIRE(Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
	}
	
	SECTION("ClearGarbage")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		Object->MarkAsGarbage();
		REQUIRE(Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));

		AutoRTFM::Testing::Commit([&]
		{
			Object->ClearGarbage();
		});
		
		REQUIRE(!Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(!Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
	}
	
	SECTION("MarkAsGarbage + ClearGarbage")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();

		AutoRTFM::Testing::Commit([&]
		{
			Object->MarkAsGarbage();
			Object->ClearGarbage();
		});

		REQUIRE(!Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(!Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
	}
	
	SECTION("Originally Garbage -> MarkAsGarbage + Abort")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		Object->MarkAsGarbage();

		// Check that we don't clear something as being garbage that was garbage before the transaction aborted!
		AutoRTFM::Testing::Abort([&]
		{
			Object->MarkAsGarbage();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
	}
	
	SECTION("ClearGarbage + Abort")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();

		// Check that we don't mark something as garbage that wasn't garbage before the transaction aborted!
		AutoRTFM::Testing::Abort([&]
		{
			Object->ClearGarbage();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(!Object->HasAnyFlags(EObjectFlags::RF_MirroredGarbage));
		REQUIRE(!Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
	}
	
	SECTION("MarkAsGarbage + Async + Abort")
	{
		UMyAutoRTFMTestObject* const Object = NewObject<UMyAutoRTFMTestObject>();
		Object->SetInternalFlags(EInternalObjectFlags::Async);

		// Check that the async flag is recovered if we abort!
		AutoRTFM::Testing::Abort([&]
		{
			Object->MarkAsGarbage();
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Object->HasAnyInternalFlags(EInternalObjectFlags::Async));
	}
}

TEST_CASE("FUObjectAnnotationSparse.AddAnnotation")
{
	struct AUTORTFM_ENABLE FTestAnnotation
	{
		FTestAnnotation()
			: TestAnnotationNumber(42)
		{
		}

		int TestAnnotationNumber;

		bool IsDefault() const
		{
			return TestAnnotationNumber == 42;
		}

		bool operator==(const FTestAnnotation& Other) const
		{
			return Other.TestAnnotationNumber == TestAnnotationNumber;
		}
	};

	FUObjectAnnotationSparse<FTestAnnotation, true> AnnotationMap;
	
	UMyAutoRTFMTestObject* Key = NewObject<UMyAutoRTFMTestObject>();
	UMyAutoRTFMTestObject* Key2 = NewObject<UMyAutoRTFMTestObject>();

	FTestAnnotation ValueA;
	ValueA.TestAnnotationNumber = 10;

	FTestAnnotation ValueB;
	ValueB.TestAnnotationNumber = 20;

	FTestAnnotation ValueC;
	ValueC.TestAnnotationNumber = 30;

	SECTION("Add")
	{
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(FTestAnnotation() == AnnotationMap.GetAnnotation(Key));
				AnnotationMap.AddAnnotation(Key, ValueA);
				REQUIRE(ValueA == AnnotationMap.GetAnnotation(Key));
			});

			REQUIRE(ValueA == AnnotationMap.GetAnnotation(Key));
		}

		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				AnnotationMap.AddAnnotation(Key, ValueA);
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(FTestAnnotation() == AnnotationMap.GetAnnotation(Key));
		}
	}

	SECTION("Replace")
	{
		AnnotationMap.AddAnnotation(Key, ValueB);

		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(ValueB == AnnotationMap.GetAnnotation(Key));
				AnnotationMap.AddAnnotation(Key, ValueA);
				REQUIRE(ValueA == AnnotationMap.GetAnnotation(Key));
			});

			REQUIRE(ValueA == AnnotationMap.GetAnnotation(Key));
		}

		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				AnnotationMap.AddAnnotation(Key, ValueA);
				AutoRTFM::AbortTransaction();
			});

			REQUIRE(ValueB == AnnotationMap.GetAnnotation(Key));
		}
	}

	SECTION("Add, Commit(Remove), Get")
	{
		AnnotationMap.AddAnnotation(Key, ValueC);

		AutoRTFM::Testing::Commit([&]
		{
			AnnotationMap.RemoveAnnotation(Key);
			REQUIRE(AnnotationMap.GetAnnotation(Key) == FTestAnnotation{});
		});

		REQUIRE(AnnotationMap.GetAnnotation(Key) == FTestAnnotation{});
	}

	SECTION("Add, Abort(Remove), Get")
	{
		AnnotationMap.AddAnnotation(Key, ValueC);

		AutoRTFM::Testing::Abort([&]
		{
			AnnotationMap.RemoveAnnotation(Key);
			REQUIRE(AnnotationMap.GetAnnotation(Key) == FTestAnnotation{});
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(AnnotationMap.GetAnnotation(Key) == ValueC);
	}

	SECTION("Add 1, Add 2, Commit(Get 1), Get 2")
	{
		AnnotationMap.AddAnnotation(Key,  ValueA);
		AnnotationMap.AddAnnotation(Key2, ValueB);

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(AnnotationMap.GetAnnotation(Key) == ValueA);
		});

		REQUIRE(AnnotationMap.GetAnnotation(Key2) == ValueB);
	}

	SECTION("Add 1, Add 2, Abort(Get 1), Get 2")
	{
		AnnotationMap.AddAnnotation(Key, ValueA);
		AnnotationMap.AddAnnotation(Key2, ValueB);

		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AnnotationMap.GetAnnotation(Key) == ValueA);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(AnnotationMap.GetAnnotation(Key2) == ValueB);
	}

	SECTION("Add 1, Add 2, Open(Get 1), Get 2")
	{
		AnnotationMap.AddAnnotation(Key, ValueA);
		AnnotationMap.AddAnnotation(Key2, ValueB);

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				REQUIRE(AnnotationMap.GetAnnotation(Key) == ValueA);
				});
		});

		REQUIRE(AnnotationMap.GetAnnotation(Key2) == ValueB);
	}
}

struct FAnnotationObject
{
	UObject* Object = nullptr;

	FAnnotationObject() {}

	FAnnotationObject(UObject* InObject) : Object(InObject) {}

	bool IsDefault() { return !Object; }
};

template <> struct TIsPODType<FAnnotationObject> { enum { Value = true }; };

TEST_CASE("UObject.AnnotationMap")
{
	FUObjectAnnotationSparse<FAnnotationObject, false> AnnotationMap;

	UObject* Key = NewObject<UMyAutoRTFMTestObject>();

	AutoRTFM::Testing::Commit([&]
	{
		UObject* Value = NewObject<UMyAutoRTFMTestObject>();
		AnnotationMap.GetAnnotation(Key);
		AnnotationMap.AddAnnotation(Key, Value);
	});

	REQUIRE(!AnnotationMap.GetAnnotation(Key).IsDefault());
}

TEST_CASE("UObject.AtomicallySetFlags")
{
	UObject* const Object = NewObject<UMyAutoRTFMTestObject>();

	constexpr EObjectFlags OldFlags = EObjectFlags::RF_Public | EObjectFlags::RF_Transient;
	constexpr EObjectFlags FlagsToAdd = EObjectFlags::RF_Transient | EObjectFlags::RF_NonPIEDuplicateTransient;

	// We need to ensure we cover the case where we are adding a flag that is already there
	// and thus cannot just wipe that out if we abort!
	Object->AtomicallyClearFlags(FlagsToAdd);
	Object->AtomicallySetFlags(OldFlags);

	REQUIRE(Object->HasAllFlags(OldFlags) & !Object->HasAllFlags(FlagsToAdd));

	AutoRTFM::Testing::Abort([&]
	{
		Object->AtomicallySetFlags(FlagsToAdd);
		AutoRTFM::AbortTransaction();
	});

	REQUIRE(Object->HasAllFlags(OldFlags) & !Object->HasAllFlags(FlagsToAdd));

	AutoRTFM::Testing::Commit([&]
	{
		Object->AtomicallySetFlags(FlagsToAdd);
	});

	REQUIRE(Object->HasAllFlags(OldFlags) & Object->HasAllFlags(FlagsToAdd));
}

TEST_CASE("UObject.AtomicallyClearFlags")
{
	UObject* const Object = NewObject<UMyAutoRTFMTestObject>();

	constexpr EObjectFlags OldFlags = EObjectFlags::RF_Public | EObjectFlags::RF_Transient;
	constexpr EObjectFlags FlagsToClear = EObjectFlags::RF_Transient | EObjectFlags::RF_NonPIEDuplicateTransient;

	// We need to ensure we cover the case where we are adding a flag that is already there
	// and thus cannot just wipe that out if we abort!
	Object->AtomicallyClearFlags(FlagsToClear);
	Object->AtomicallySetFlags(OldFlags);

	REQUIRE(Object->HasAllFlags(OldFlags) & !Object->HasAllFlags(FlagsToClear));

	AutoRTFM::Testing::Abort([&]
	{
		Object->AtomicallyClearFlags(FlagsToClear);
		AutoRTFM::AbortTransaction();
	});

	REQUIRE(Object->HasAllFlags(OldFlags) & !Object->HasAllFlags(FlagsToClear));

	AutoRTFM::Testing::Commit([&]
	{
		Object->AtomicallyClearFlags(FlagsToClear);
	});

	REQUIRE(Object->HasAnyFlags(OldFlags) & !Object->HasAllFlags(FlagsToClear));
}

// Tests that constructing a UObject in both the open and closed doesn't result
// in a corrupt FUObjectThreadContext. See SOL-7131.
TEST_CASE("UObject.FUObjectThreadContext")
{
	AutoRTFM::TScopedGuard<UMyAutoRTFMTestObject::FConstructorCallback*> CallbackScope(UMyAutoRTFMTestObject::ConstructorCallback, nullptr);

	struct AUTORTFM_ENABLE Fns
	{
		static void CreateObjectWithCtor(UMyAutoRTFMTestObject::FConstructorCallback* Ctor)
		{
			AutoRTFM::Open([&] { UMyAutoRTFMTestObject::ConstructorCallback = Ctor; });
			NewObject<UMyAutoRTFMTestObject>();
		}
		
		static void CtorCreateInnerClosed(const FObjectInitializer& ObjectInitializer, UMyAutoRTFMTestObject& Object)
		{
			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				REQUIRE(1 == FUObjectThreadContext::Get().IsInConstructor);
				CreateObjectWithCtor(nullptr);
				REQUIRE(1 == FUObjectThreadContext::Get().IsInConstructor);
			});
			REQUIRE(AutoRTFM::ETransactionStatus::Executing == Status);
		}
		
		static void CtorCreateInnerTransact(const FObjectInitializer& ObjectInitializer, UMyAutoRTFMTestObject& Object)
		{
			AutoRTFM::Testing::Commit([&]
			{
				REQUIRE(1 == FUObjectThreadContext::Get().IsInConstructor);
				CreateObjectWithCtor(nullptr);
				REQUIRE(1 == FUObjectThreadContext::Get().IsInConstructor);
			});
		}

		static void CtorAbort(const FObjectInitializer& ObjectInitializer, UMyAutoRTFMTestObject& Object)
		{
			REQUIRE(1 == FUObjectThreadContext::Get().IsInConstructor);
			AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
			{
				AutoRTFM::AbortTransaction();
				FAIL(/* unreachable */);
			});
			REQUIRE(AutoRTFM::ETransactionStatus::AbortedByRequest == Status);
		}
	};

	SECTION("Transact(UObjectCtor(Abort))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			Fns::CreateObjectWithCtor(Fns::CtorAbort);
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}
	SECTION("Transact(UObjectCtor(), Abort)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
				
			NewObject<UMyAutoRTFMTestObject>();

			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			AutoRTFM::AbortTransaction();
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}

	SECTION("Transact(Open(UObjectCtor(Transact(UObjectCtor))), UObjectCtor(Abort))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
				
			AutoRTFM::Open([&]
			{
				Fns::CreateObjectWithCtor(Fns::CtorCreateInnerTransact);
			});

			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			Fns::CreateObjectWithCtor(Fns::CtorAbort);
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}

	SECTION("Transact(Open(UObjectCtor(Transact(UObjectCtor))), Abort)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
				
			AutoRTFM::Open([&]
			{
				Fns::CreateObjectWithCtor(Fns::CtorCreateInnerTransact);
			});

			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			AutoRTFM::AbortTransaction();
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}

	SECTION("Transact(Open(UObjectCtor(Close(UObjectCtor))), UObjectCtor(Abort))")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
				
			AutoRTFM::Open([&]
			{
				Fns::CreateObjectWithCtor(Fns::CtorCreateInnerClosed);
			});

			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			Fns::CreateObjectWithCtor(Fns::CtorAbort);
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}
	
	SECTION("Transact(Open(UObjectCtor(Close(UObjectCtor))), Abort)")
	{
		AutoRTFM::Testing::Abort([&]
		{
			REQUIRE(AutoRTFM::IsClosed());
			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
				
			AutoRTFM::Open([&]
			{
				Fns::CreateObjectWithCtor(Fns::CtorCreateInnerClosed);
			});

			REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);

			AutoRTFM::AbortTransaction();
			FAIL(/* unreachable */);
		});

		REQUIRE(0 == FUObjectThreadContext::Get().IsInConstructor);
	}
}

TEST_CASE("UObject.AddRef")
{
	SECTION("Default")
	{
		UMyAutoRTFMTestObject* Object = NewObject<UMyAutoRTFMTestObject>();

		AutoRTFM::Testing::Commit([&]
		{
			Object->AddRef();
		});

		Object->ReleaseRef();
	}
}

TEST_CASE("UObject.ReleaseRef")
{
	SECTION("Default")
	{
		UMyAutoRTFMTestObject* Object = NewObject<UMyAutoRTFMTestObject>();

		Object->AddRef();

		AutoRTFM::Testing::Commit([&]
		{
			Object->ReleaseRef();
		});
	}

	SECTION("With Cascading Retry")
	{
		UMyAutoRTFMTestObject* Object = NewObject<UMyAutoRTFMTestObject>();

		Object->AddRef();

		bool bFirst = true;

		AutoRTFM::Testing::Commit([&]
		{
			Object->ReleaseRef();

			if (bFirst)
			{
				AutoRTFM::OnRetry([&] 
				{ 
					bFirst = false; 
				});
				AutoRTFM::CascadingRetryTransaction();
			}
		});
	}
}

TEST_CASE("TObjectPtr")
{
	UMyAutoRTFMTestObject* Object = NewObject<UMyAutoRTFMTestObject>();

	SECTION("Construct")
	{
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				TObjectPtr<UMyAutoRTFMTestObject> ObjectPtr(Object);
			});
		}

		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				TObjectPtr<UMyAutoRTFMTestObject> ObjectPtr(Object);
				AutoRTFM::AbortTransaction();
			});
		}
	}

	SECTION("Destruct")
	{
		std::optional<TObjectPtr<UMyAutoRTFMTestObject>> ObjectPtr = TObjectPtr<UMyAutoRTFMTestObject>(Object);
		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				ObjectPtr.reset();
			});
		}

		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				ObjectPtr.reset();
				AutoRTFM::AbortTransaction();
			});
		}
	}
}

TEST_CASE("FGCObject")
{
	struct AUTORTFM_ENABLE FMyGCObject : FGCObject
	{
		bool bAlive = true;

		~FMyGCObject()
		{
			REQUIRE(bAlive);
			bAlive = false;
		}

		void AddReferencedObjects([[maybe_unused]] FReferenceCollector& Collector) override
		{
			REQUIRE(bAlive);
		}

		FString GetReferencerName() const override
		{
			REQUIRE(bAlive);
			return {};
		}
	};

	auto Test = [&] AUTORTFM_ENABLE (AUTORTFM_IMPLICIT_ENABLE auto&& Callback)
	{
		SECTION("NonTransactional")
		{
			Callback();
		}

		SECTION("Commit")
		{
			AutoRTFM::Testing::Commit([&]
			{
				Callback();
			});
		}

		SECTION("Abort")
		{
			AutoRTFM::Testing::Abort([&]
			{
				Callback();
				AutoRTFM::AbortTransaction();
			});
		}
	};

	// Tests for FGCObject that is allocated on the heap.
	SECTION("Heap")
	{
		SECTION("Transact(Construct, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			});
		}

		SECTION("Transact(Construct, Unregister, Register, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Unregister, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
				Object->UnregisterGCObject();
			});
		}

		SECTION("Transact(Construct, Register, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
				Object->RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Register, Unregister, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
				Object->RegisterGCObject();
				Object->UnregisterGCObject();
			});
		}

		SECTION("Transact(Construct, Unregister, Unregister, Register, Register, Destruct)")
		{
			Test([&]
			{
				TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
				Object->UnregisterGCObject();
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
				Object->RegisterGCObject();
			});
		}

		SECTION("Transact(Construct), Destruct")
		{
			TUniquePtr<FMyGCObject> Object;
			Test([&]
			{
				Object = MakeUnique<FMyGCObject>();
			});
		}

		SECTION("Construct, Transact(Destruct)")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object.Reset();
			});
		}

		SECTION("Construct, Transact(Unregister), Destruct")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
			});
		}

		SECTION("Construct, Transact(Unregister, Register), Destruct")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
			});
		}

		SECTION("Construct, Transact(Unregister, Destruct)")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Object.Reset();
			});
		}

		SECTION("Construct, Transact(Unregister, Register, Destruct)")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
				Object.Reset();
			});
		}

		SECTION("Construct, Transact(Unregister, Transact(Register)), Destruct")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Test([&]
				{
					Object->RegisterGCObject();
				});
			});
		}

		SECTION("Construct, Transact(Unregister, Register, Transact(Unregister)), Destruct")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
				Test([&]
				{
					Object->UnregisterGCObject();
				});
			});
		}

		SECTION("Construct, Transact(Unregister, Transact(Register), Destruct)")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Test([&]
				{
					Object->RegisterGCObject();
				});
				Object.Reset();
			});
		}

		SECTION("Construct, Transact(Unregister, Register, Transact(Unregister), Destruct)")
		{
			TUniquePtr<FMyGCObject> Object = MakeUnique<FMyGCObject>();
			Test([&]
			{
				Object->UnregisterGCObject();
				Object->RegisterGCObject();
				Test([&]
				{
					Object->UnregisterGCObject();
				});
				Object.Reset();
			});
		}
	}

	// Tests for FGCObject that is allocated within the transaction's stack.
	SECTION("Stack")
	{
		SECTION("Transact(Construct, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
			});
		}

		SECTION("Transact(Construct, Unregister, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.UnregisterGCObject();
			});
		}

		SECTION("Transact(Construct, Unregister, Unregister, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.UnregisterGCObject();
				MyGCObject.UnregisterGCObject();
				MyGCObject.RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Unregister, Register, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.UnregisterGCObject();
				MyGCObject.RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Register, Register, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.RegisterGCObject();
				MyGCObject.RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Register, Unregister, Register, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.RegisterGCObject();
				MyGCObject.UnregisterGCObject();
				MyGCObject.RegisterGCObject();
			});
		}

		SECTION("Transact(Construct, Transact(Unregister, Register), Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				Test([&]
				{
					MyGCObject.UnregisterGCObject();
					MyGCObject.RegisterGCObject();
				});
			});
		}

		SECTION("Transact(Construct, Unregister, Transact(Register), Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				MyGCObject.UnregisterGCObject();
				Test([&]
				{
					MyGCObject.RegisterGCObject();
				});
			});
		}

		SECTION("Transact(Construct, Transact(Unregister), Register, Destruct)")
		{
			Test([&]
			{
				FMyGCObject MyGCObject;
				Test([&]
				{
					MyGCObject.UnregisterGCObject();
				});
				MyGCObject.RegisterGCObject();
			});
		}

		SECTION("Construct, Transact(Destruct)")
		{
			std::optional<FMyGCObject> MyGCObject{FMyGCObject{}};
			Test([&]
			{
				MyGCObject.reset();
			});
		}

		SECTION("Transact(Construct), Destruct")
		{
			std::optional<FMyGCObject> MyGCObject;
			Test([&]
			{
				MyGCObject = FMyGCObject{};
			});
		}

		SECTION("Construct, Transact(Unregister, Destruct)")
		{
			std::optional<FMyGCObject> MyGCObject{FMyGCObject{}};
			Test([&]
			{
				MyGCObject->UnregisterGCObject();
				MyGCObject.reset();
			});
		}

		SECTION("Transact(Construct, Unregister), Destruct")
		{
			std::optional<FMyGCObject> MyGCObject;
			Test([&]
			{
				MyGCObject = FMyGCObject{};
				MyGCObject->UnregisterGCObject();
			});
		}
	}
}

TEST_CASE("UObject.ObjectWithSubObject")
{
	UMyAutoRTFMTestObject* Outer = NewObject<UMyAutoRTFMTestObject>();
	UMyAutoRTFMTestObjectWithSubObjects* Object = nullptr;

	bool bRetry = true;

	AutoRTFM::Testing::Commit([&]
	{
		Object = NewObject<UMyAutoRTFMTestObjectWithSubObjects>(Outer, TEXT("WOWWEE"));
		REQUIRE(Object);
	});
}

TEST_CASE("UObject.FUniversallyUniqueObjectNameGenerator")
{
	SECTION("Constructed outside of transaction")
	{
		FGuid Seed;
		FUniversallyUniqueObjectNameGenerator Generator(Seed);

		const uint32 Value = Generator.GetNewValue();

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Generator.GetNewValue() != Value);
		});
	}

	SECTION("Constructed inside of transaction")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FGuid Seed;
			FUniversallyUniqueObjectNameGenerator Generator(Seed);
			uint32 Value = Generator.GetNewValue();

			// Just check that we are definitely generating the pseudo-random values even in transactions.
			REQUIRE(Value != Generator.GetNewValue());
		});
	}

	SECTION("Seed is not reset by aborting transaction")
	{
		FGuid Seed;
		FUniversallyUniqueObjectNameGenerator Generator(Seed);

		uint32 Value = 0;

		AutoRTFM::Testing::Abort([&]
		{
			const uint32 TransactionalValue = Generator.GetNewValue();

			AutoRTFM::OnAbort([&Value, TransactionalValue]
			{
				Value = TransactionalValue;
			});

			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Value != Generator.GetNewValue());
	}
}
