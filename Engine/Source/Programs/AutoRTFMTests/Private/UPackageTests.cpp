// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFMTestUtils.h"
#include "AutoRTFM/Testing.h"
#include "MyAutoRTFMTestObject.h"
#include "Misc/PackageName.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobalsInternal.h"

#include "Catch2Includes.h"

TEST_CASE("UPackage.SetPackageFlagsTo")
{
	SECTION("Commit")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_None);

		AutoRTFM::Testing::Commit([&]
		{
			Package->SetPackageFlagsTo(PKG_TransientFlags);
		});

		REQUIRE(Package->GetPackageFlags() == PKG_TransientFlags);
	}

	SECTION("Abort")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_None);

		AutoRTFM::Testing::Abort([&]
		{
			Package->SetPackageFlagsTo(PKG_TransientFlags);
			REQUIRE(Package->GetPackageFlags() == PKG_TransientFlags);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Package->GetPackageFlags() == PKG_None);
	}
}

TEST_CASE("UPackage.SetPackageFlags")
{
	SECTION("Commit")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_RuntimeGenerated);

		AutoRTFM::Testing::Commit([&]
		{
			Package->SetPackageFlags(PKG_TransientFlags);
		});

		REQUIRE(Package->GetPackageFlags() == (PKG_RuntimeGenerated | PKG_TransientFlags));
	}

	SECTION("Abort")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_RuntimeGenerated);

		AutoRTFM::Testing::Abort([&]
		{
			Package->SetPackageFlags(PKG_TransientFlags);
			REQUIRE(Package->GetPackageFlags() == (PKG_RuntimeGenerated | PKG_TransientFlags));
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Package->GetPackageFlags() == PKG_RuntimeGenerated);
	}
}

TEST_CASE("UPackage.ClearPackageFlags")
{
	SECTION("Commit")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_RuntimeGenerated | PKG_TransientFlags);

		AutoRTFM::Testing::Commit([&]
		{
			Package->ClearPackageFlags(PKG_TransientFlags);
		});

		REQUIRE(Package->GetPackageFlags() == PKG_RuntimeGenerated);
	}

	SECTION("Abort")
	{
		UPackage* Package = NewObject<UPackage>();
		Package->SetPackageFlagsTo(PKG_RuntimeGenerated | PKG_TransientFlags);

		AutoRTFM::Testing::Abort([&]
		{
			Package->ClearPackageFlags(PKG_TransientFlags);
			REQUIRE(Package->GetPackageFlags() == PKG_RuntimeGenerated);
			AutoRTFM::AbortTransaction();
		});

		REQUIRE(Package->GetPackageFlags() == (PKG_RuntimeGenerated | PKG_TransientFlags));
	}
}

static const TCHAR* ObjectName = TEXT("TestObject");

// Based on `Engine\Private\Tests\Loading\AsyncLoadingTests_Shared.h`, we use similar logic
// here to make a package that the loader will see and be able to actually load!
enum class EShouldDeferPackageCreation : bool
{
	Yes = true,
	No = false,
};

struct AUTORTFM_DISABLE FPackageScopedMaker final
{
	FString PackageName;
	FString PackagePath;
	EShouldDeferPackageCreation ShouldDeferPackageCreation;

	FPackageScopedMaker(FString InPackageName, EShouldDeferPackageCreation InShouldDeferPackageCreation = EShouldDeferPackageCreation::No) :
		PackageName(InPackageName),
		PackagePath(FPackageName::LongPackageNameToFilename(*InPackageName, FPackageName::GetAssetPackageExtension())),
		ShouldDeferPackageCreation(InShouldDeferPackageCreation)
	{
		// We need to remove any previous package of the same name (could have occurred if a previous test ran segfaulted for instance).
		if (FPackageName::DoesPackageExist(PackageName))
		{
			REQUIRE(IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackagePath, false));
			REQUIRE(IPlatformFile::GetPlatformPhysical().DeleteFile(*PackagePath));
		}

		if (ShouldDeferPackageCreation == EShouldDeferPackageCreation::No)
		{
			// `MakePackage` will assert that we were deferring package creation (so that if a user calls it later,
			// they actually asked to defer package creation before hand!). So we just switch it to yes to let
			// `MakePackage` succeed, which will switch it back anyways.
			ShouldDeferPackageCreation = EShouldDeferPackageCreation::Yes;
			MakePackage();
		}
	}

	void MakePackage()
	{
		check(ShouldDeferPackageCreation == EShouldDeferPackageCreation::Yes);

		// Ensure that async loading is done.
		FlushAsyncLoading();

		// Create a package.
		UPackage* const Package = CreatePackage(*PackageName);

		UObject* const Object = NewObject<UMyAutoRTFMTestObject>(Package, ObjectName, RF_Public | RF_Standalone);

		// Need to mark it is loaded.
		Package->MarkAsFullyLoaded();

		// Save the package to the file-system.
		REQUIRE(UPackage::SavePackage(Package, Object, *PackagePath, FSavePackageArgs()));

		// Make sure the package existed in our tables before.
		REQUIRE(FindObject<UObject>(nullptr, *PackageName) != nullptr);

		// GC and make sure everything gets cleaned up before loading.
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		// Then make sure the package is no longer loaded in our tables after.
		REQUIRE(FindObject<UObject>(nullptr, *PackageName) == nullptr);

		ShouldDeferPackageCreation = EShouldDeferPackageCreation::No;
	}
};

TEST_CASE("UPackage.AsyncLoading")
{
	SECTION("DoesPackageExist")
	{
		AutoRTFM::Testing::Commit([&]
		{
			FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
			REQUIRE(!FPackageName::DoesPackageExist(Name));
		});
		
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(FPackageName::DoesPackageExist(Name));
		});
	}

	SECTION("LoadPackageAsync")
	{
		int32 RequestId = -1;

		AutoRTFM::Testing::Commit([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			RequestId = LoadPackageAsync(Name);
		});

		FlushAsyncLoading(RequestId);
	}

	SECTION("IsAsyncLoading")
	{
		int32 RequestId = -1;

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(!IsAsyncLoading());

			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			RequestId = LoadPackageAsync(Name);

			REQUIRE(IsAsyncLoading());
		});

		FlushAsyncLoading(RequestId);
	}

	SECTION("FlushAsyncLoading")
	{
		FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
		int32 RequestId = LoadPackageAsync(Name);

		AutoRTFM::Testing::Commit([&]
		{
			FlushAsyncLoading(RequestId);
		});
	}

	SECTION("FlushAsyncLoading Empty")
	{
		FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
		int32 RequestId = LoadPackageAsync(Name);

		AutoRTFM::Testing::Commit([&]
		{
			FlushAsyncLoading();
		});
	}

	SECTION("FlushAsyncLoading One In One Out")
	{
		FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
		int32 RequestId1 = LoadPackageAsync(Name);

		AutoRTFM::Testing::Commit([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			int32 RequestId2 = LoadPackageAsync(Name);
			TArray<int32> RequestIds;
			RequestIds.Add(RequestId1);
			RequestIds.Add(RequestId2);
			FlushAsyncLoading(RequestIds);
		});
	}

	SECTION("CompletionDelegate is called closed")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) { REQUIRE(AutoRTFM::IsClosed()); });
			int RequestId = LoadPackageAsync(Name, CompletionDelegate);
			FlushAsyncLoading(RequestId);
			AutoRTFM::AbortTransaction();
		});
	}
	
	SECTION("CompletionDelegate aborts")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) { AutoRTFM::AbortTransaction(); });
			int RequestId = LoadPackageAsync(Name, CompletionDelegate);
			FlushAsyncLoading(RequestId);
			FAIL("Unreachable!");
		});
	}
	
	SECTION("FLoadPackageAsyncOptionalParams::CompletionDelegate is called closed")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			FLoadPackageAsyncOptionalParams Params;
			Params.CompletionDelegate.Reset(new FLoadPackageAsyncDelegate());
			Params.CompletionDelegate->BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) { REQUIRE(AutoRTFM::IsClosed()); });
			int RequestId = LoadPackageAsync(Name, MoveTemp(Params));
			FlushAsyncLoading(RequestId);
			AutoRTFM::AbortTransaction();
		});
	}
	
	SECTION("FLoadPackageAsyncOptionalParams::CompletionDelegate aborts")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			FLoadPackageAsyncOptionalParams Params;
			Params.CompletionDelegate.Reset(new FLoadPackageAsyncDelegate());
			Params.CompletionDelegate->BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) { AutoRTFM::AbortTransaction(); });
			int RequestId = LoadPackageAsync(Name, MoveTemp(Params));
			FlushAsyncLoading(RequestId);
			FAIL("Unreachable!");
		});
	}
	
	SECTION("FLoadPackageAsyncOptionalParams::CompletionDelegate creates UObject")
	{
		UMyAutoRTFMTestObject* OpenObject = nullptr;
		UMyAutoRTFMTestObject* ClosedObject = nullptr;

		AutoRTFM::Testing::Abort([&]
		{
			ClosedObject = NewObject<UMyAutoRTFMTestObject>();
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			FLoadPackageAsyncOptionalParams Params;
			Params.CompletionDelegate.Reset(new FLoadPackageAsyncDelegate());
			Params.CompletionDelegate->BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type)
			{
				OpenObject = NewObject<UMyAutoRTFMTestObject>();
				AutoRTFM::AbortTransaction();
			});
			int RequestId = LoadPackageAsync(Name, MoveTemp(Params));
			FlushAsyncLoading(RequestId);
			FAIL("Unreachable!");
		});

		REQUIRE(nullptr == ClosedObject);
		REQUIRE(nullptr == OpenObject);
	}
	
	SECTION("FLoadPackageAsyncOptionalParams::CompletionDelegate calls another LoadPackageAsync")
	{
		AutoRTFM::Testing::Abort([&]
		{
			FLoadPackageAsyncOptionalParams Params;
			Params.CompletionDelegate.Reset(new FLoadPackageAsyncDelegate());
			Params.CompletionDelegate->BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type)
			{
				int RequestId = LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), MoveTemp(Params));
				AutoRTFM::AbortTransaction();
			});
			int RequestId = LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), MoveTemp(Params));
			FlushAsyncLoading(RequestId);
			FAIL("Unreachable!");
		});
	}

	SECTION("Multiple retries because of multiple loads with commit")
	{
		AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

		int NumCompletionCallbacks = 0;
		AutoRTFM::Testing::Commit([&]
		{
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) 
			{
				// Do this open so we can check how many retries occurred.
				AutoRTFM::Open([&] { NumCompletionCallbacks++; });
			});

			TArray<int32> RequestIds;
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));

			FlushAsyncLoading(RequestIds);
			REQUIRE(3 == NumCompletionCallbacks);
		});
	}

	SECTION("Multiple retries because of multiple loads with abort")
	{
		AutoRTFMTestUtils::FScopedRetry Retry(AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry);

		int NumCompletionCallbacks = 0;
		AutoRTFM::Testing::Abort([&]
		{
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type) 
			{
				// Do this open so we can check how many retries occurred.
				AutoRTFM::Open([&] { NumCompletionCallbacks++; });
			});

			TArray<int32> RequestIds;
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));
			RequestIds.Add(LoadPackageAsync(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__), CompletionDelegate));

			FlushAsyncLoading(RequestIds);
			REQUIRE(3 == NumCompletionCallbacks);

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("Stack Local Linker Instancing Context")
	{
		int32 RequestId = -1;

		AutoRTFM::Testing::Commit([&]
		{
			FLinkerInstancingContext Context;
			FLoadPackageAsyncOptionalParams Params;
			Params.InstancingContext = &Context;
			FString Name(FString::Printf(TEXT("/AutoRTFMTestPackage%d"), __LINE__));
			RequestId = LoadPackageAsync(Name, MoveTemp(Params));
		});

		FlushAsyncLoading(RequestId);
	}

	SECTION("Trashed Package")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
		REQUIRE(Package);
			
		TrashObject(Package);

		AutoRTFM::Testing::Commit([&]
		{
			UPackage* const ReloadedPackage = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(ReloadedPackage);
			REQUIRE(ReloadedPackage->GetFName() == Name);

			REQUIRE(Package != ReloadedPackage);
		});
	}

	SECTION("Find Package Loaded In Transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			// Make sure the package doesn't exist.
			REQUIRE(FindObject<UObject>(nullptr, *Name) == nullptr);

			UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(Package);

			TArray<UObject*> Objects;

			REQUIRE(StaticFindAllObjects(Objects, UObject::StaticClass(), *Name));
			REQUIRE(Objects.Num() == 1);
			REQUIRE(Objects[0] == Package);
		});
	}

	SECTION("Trash Package that was created in same transaction as reloaded")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(Package);
			
			TrashObject(Package);

			UPackage* const ReloadedPackage = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(ReloadedPackage);
			REQUIRE(ReloadedPackage->GetFName() == Name);

			REQUIRE(Package != ReloadedPackage);
		});
	}

	SECTION("Trash Package that was found in the same transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);
		UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
		REQUIRE(Package);

		AutoRTFM::Testing::Commit([&]
		{
			UPackage* const FoundPackage = FindObject<UPackage>(nullptr, *Name);
			REQUIRE(Package == FoundPackage);
			
			TrashObject(Package);

			UPackage* const ReloadedPackage = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(ReloadedPackage);
			REQUIRE(ReloadedPackage->GetFName() == Name);

			REQUIRE(Package != ReloadedPackage);
		});

		UPackage* const FoundPackage = FindObject<UPackage>(nullptr, *Name);
		REQUIRE(Package != FoundPackage);
	}

	SECTION("Multiple retries to load multiple packages")
	{
		FString Name1(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _1(Name1);

		FString Name2(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _2(Name2);

		FString Name3(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _3(Name3);

		FString Name4(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _4(Name4);

		TArray<FString> PackagesToLoad({Name1, Name2, Name3, Name4});

		AutoRTFM::Testing::Commit([&]
		{
			for (const FString& Name : PackagesToLoad)
			{
				UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
				REQUIRE(Package);
			}
		});
	}

	SECTION("Multiple retries to load multiple packages with trashing")
	{
		FString Name1(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _1(Name1);

		FString Name2(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _2(Name2);

		FString Name3(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _3(Name3);

		FString Name4(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _4(Name4);

		TArray<FString> PackagesToLoad({Name1, Name2, Name3, Name4});

		AutoRTFM::Testing::Commit([&]
		{
			for (const FString& Name : PackagesToLoad)
			{
				UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
				REQUIRE(Package);

				TrashObject(Package);

				UPackage* const ReloadedPackage = LoadPackage(nullptr, *Name, LOAD_None);
				REQUIRE(ReloadedPackage);
				REQUIRE(ReloadedPackage->GetFName() == Name);

				REQUIRE(Package != ReloadedPackage);
			}
		});
	}

	SECTION("Multiple async loads")
	{
		FString Name1(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _1(Name1);

		FString Name2(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _2(Name2);

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(FindObject<UPackage>(nullptr, *Name1) == nullptr);
			REQUIRE(FindObject<UPackage>(nullptr, *Name2) == nullptr);

			FLoadPackageAsyncDelegate Delegate;
			const int32 Id1 = LoadPackageAsync(Name1, Delegate);
			const int32 Id2 = LoadPackageAsync(Name2, Delegate);
			FlushAsyncLoading(Id2);
			
			REQUIRE(FindObject<UPackage>(nullptr, *Name2) != nullptr);
		});
	}

	SECTION("Load With Custom Name")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			FName CustomName("CustomName", 1);

			FLoadPackageAsyncDelegate Delegate;

			Delegate.BindLambda([&](const FName& Name, UPackage* const Package, EAsyncLoadingResult::Type Result)
			{
				REQUIRE(EAsyncLoadingResult::Succeeded == Result);
				REQUIRE(Package->GetFName() == Name);
				REQUIRE(CustomName == Name);
			});

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), CustomName, Delegate));
		});
	}

	SECTION("Trash And Load With Custom Name")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		bool bHit = false;

		UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
		REQUIRE(Package);

		AutoRTFM::Testing::Commit([&]
		{
			TrashObject(Package);

			FName CustomName("CustomName", 2);

			FLoadPackageAsyncDelegate Delegate;

			Delegate.BindLambda([&](const FName& Name, UPackage* const Package, EAsyncLoadingResult::Type Result)
			{
				REQUIRE(AutoRTFM::IsClosed());
				REQUIRE(EAsyncLoadingResult::Succeeded == Result);
				REQUIRE(Package->GetFName() == Name);
				REQUIRE(CustomName == Name);
				bHit = true;
			});

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), CustomName, Delegate));
		});

		REQUIRE(bHit);
	}

	SECTION("Transactionally allocated shared pointer captured by value in completion delegate")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Transact([&]
		{
			TSharedRef<bool> TransactionallyCreatedPtr = MakeShared<bool>(false);

			bool bHit = false;
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&bHit, TransactionallyCreatedPtr](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}

	SECTION("Transactionally allocated shared pointer to shared from this captured by value in completion delegate")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Transact([&]
		{
			struct FThing final : TSharedFromThis<FThing>
			{
				FThing(bool bInMember) : bMember(bInMember) {}
				bool bMember = false;
			};
			TSharedRef<FThing> TransactionallyCreatedPtr = MakeShared<FThing>(false);

			bool bHit = false;
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&bHit, TransactionallyCreatedPtr](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}

	SECTION("Transactionally allocated weak pointer captured by value in completion delegate")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Transact([&]
		{
			TSharedRef<bool> TransactionallyCreatedPtr = MakeShared<bool>(false);

			TWeakPtr<bool> WeakPtr = TransactionallyCreatedPtr;

			bool bHit = false;
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&bHit, WeakPtr](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}

	SECTION("Transactionally allocated weak pointer to shared from this captured by value in completion delegate")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Transact([&]
		{
			struct FThing final : TSharedFromThis<FThing>
			{
				FThing(bool bInMember) : bMember(bInMember) {}
				bool bMember = false;
			};
			TSharedRef<FThing> TransactionallyCreatedPtr = MakeShared<FThing>(false);

			TWeakPtr<FThing> WeakPtr = TransactionallyCreatedPtr;

			bool bHit = false;
			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&bHit, WeakPtr](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}

	SECTION("Transactionally allocated shared pointer captured by value in completion delegate in a child")
	{
		FString Name1(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _1(Name1);

		FString Name2(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _2(Name2);

		AutoRTFM::Transact([&]
		{
			FLoadPackageAsyncDelegate CompletionDelegate;
			LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name1), NAME_None, CompletionDelegate);

			AutoRTFM::Transact([&]
			{
				TSharedRef<bool> TransactionallyCreatedPtr = MakeShared<bool>(false);

				FLoadPackageAsyncDelegate CompletionDelegate;
				CompletionDelegate.BindLambda([TransactionallyCreatedPtr](const FName&, UPackage*, EAsyncLoadingResult::Type Type) {});

				LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name2), NAME_None, CompletionDelegate);

				AutoRTFM::AbortTransaction();
			});

			AutoRTFM::AbortTransaction();
		});
	}

	SECTION("First Time Load Package, Second Time Don't")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		bool bHit = false;

		bool bFirst = true;

		FName CustomName("CustomName", 3);

		FLoadPackageAsyncDelegate Delegate;

		Delegate.BindLambda([&](const FName& Name, UPackage* const Package, EAsyncLoadingResult::Type Result)
		{
			REQUIRE(!AutoRTFM::IsTransactional());
			REQUIRE(EAsyncLoadingResult::Succeeded == Result);
			REQUIRE(Package->GetFName() == Name);
			REQUIRE(CustomName == Name);
			bHit = true;
		});

		AutoRTFM::Testing::Commit([&]
		{
			if (!bFirst)
			{
				return;
			}

			AutoRTFM::OnRetry([&] { bFirst = false; });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), CustomName, Delegate));
		});

		FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), CustomName, Delegate));

		REQUIRE(bHit);
	}

	SECTION("Object is correct during trashing")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
		REQUIRE(Package);

		UMyAutoRTFMTestObject* const Object = FindObject<UMyAutoRTFMTestObject>(Package, ObjectName);
		REQUIRE(Object);

		REQUIRE(Object->Value == 42);

		Object->Value = 13;

		AutoRTFM::Testing::Commit([&]
		{
			REQUIRE(Object->Value == 13);
			TrashObject(Package);

			UPackage* const ReloadedPackage = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(ReloadedPackage);

			UMyAutoRTFMTestObject* const ReloadedObject = FindObject<UMyAutoRTFMTestObject>(ReloadedPackage, ObjectName);
			REQUIRE(ReloadedObject);

			REQUIRE(ReloadedObject->Value == 42);
		});
	}

	SECTION("Object has correct name")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
		REQUIRE(Package);

		FString ObjectPath = Name;
		ObjectPath.Append(TEXT("."));
		ObjectPath.Append(ObjectName);

		UMyAutoRTFMTestObject* const Object = FindObject<UMyAutoRTFMTestObject>(nullptr, ObjectPath);
		REQUIRE(Object);
	}

	SECTION("Object has correct name in transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(Package);

			FString ObjectPath = Name;
			ObjectPath.Append(TEXT("."));
			ObjectPath.Append(ObjectName);

			UMyAutoRTFMTestObject* const Object = FindObject<UMyAutoRTFMTestObject>(nullptr, ObjectPath);
			REQUIRE(Object);
		});
	}

	SECTION("Create unrelated UObject's before package load")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			// A completely unrelated UObject to the package.
			UPackage* const UnrelatedPackage = NewObject<UPackage>();
			REQUIRE(UnrelatedPackage);

			UPackage* const Package = LoadPackage(nullptr, *Name, LOAD_None);
			REQUIRE(Package);
		});
	}

	SECTION("Flush In Child Transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));
		FPackageScopedMaker _(Name);

		AutoRTFM::Testing::Commit([&]
		{
			FLoadPackageAsyncDelegate Delegate;
			int32 RequestId = LoadPackageAsync(*Name, Delegate);

			AutoRTFM::Testing::Commit([&]
			{
				FlushAsyncLoading(RequestId);
			});
		});
	}

	SECTION("Packaged Load Fails First Time")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));

		FPackageScopedMaker Scope(Name, EShouldDeferPackageCreation::Yes);

		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::FailedMissing); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		}

		Scope.MakePackage();

		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		}
	}

	SECTION("Packaged Load Fails First Time, Second Try in Transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));

		FPackageScopedMaker Scope(Name, EShouldDeferPackageCreation::Yes);

		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::FailedMissing); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		}

		Scope.MakePackage();

		AutoRTFM::Transact([&]
		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}

	SECTION("Packaged Load Fails First Time In Transaction, Second Try in Transaction")
	{
		FString Name(FString::Printf(TEXT("/Game/%dAutoRTFMTestPackage%d"), FPlatformProcess::GetCurrentProcessId(), __LINE__));

		FPackageScopedMaker Scope(Name, EShouldDeferPackageCreation::Yes);

		AutoRTFM::Transact([&]
		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::FailedMissing); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});

		Scope.MakePackage();

		AutoRTFM::Transact([&]
		{
			bool bHit = false;

			FLoadPackageAsyncDelegate CompletionDelegate;
			CompletionDelegate.BindLambda([&](const FName&, UPackage*, EAsyncLoadingResult::Type Type) { bHit = true; REQUIRE(Type == EAsyncLoadingResult::Succeeded); });

			FlushAsyncLoading(LoadPackageAsync(FPackagePath::FromPackageNameUnchecked(*Name), NAME_None, CompletionDelegate));

			REQUIRE(bHit);
		});
	}
}
