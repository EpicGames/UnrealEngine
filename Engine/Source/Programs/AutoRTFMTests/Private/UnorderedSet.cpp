// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "Catch2Includes.h"

#include <unordered_set>

using namespace std;

TEST_CASE("UnorderedSet")
{
    constexpr unsigned Count = 100;
    unordered_set<unsigned> MySet;
    AutoRTFM::Testing::Commit([&] ()
    {
        unordered_set<unsigned> MySetInner;
        for (unsigned Index = Count; Index--;)
        {
            MySetInner.insert(Index);
        }

        MySet = MySetInner;
    });

    REQUIRE(MySet.size() == Count);
    for (unsigned Index = Count; Index--;)
    {
        REQUIRE(MySet.count(Index) == 1);
    }
}
