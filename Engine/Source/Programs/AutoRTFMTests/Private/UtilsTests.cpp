// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils.h"
#include "Catch2Includes.h"

TEST_CASE("AlignDown")
{
	REQUIRE(AutoRTFM::AlignDown(0, 1) == 0);
	REQUIRE(AutoRTFM::AlignDown(1, 1) == 1);
	REQUIRE(AutoRTFM::AlignDown(2, 1) == 2);
	REQUIRE(AutoRTFM::AlignDown(3, 1) == 3);
	REQUIRE(AutoRTFM::AlignDown(4, 1) == 4);
	REQUIRE(AutoRTFM::AlignDown(5, 1) == 5);
	REQUIRE(AutoRTFM::AlignDown(6, 1) == 6);
	REQUIRE(AutoRTFM::AlignDown(7, 1) == 7);
	REQUIRE(AutoRTFM::AlignDown(8, 1) == 8);

	REQUIRE(AutoRTFM::AlignDown(0, 2) == 0);
	REQUIRE(AutoRTFM::AlignDown(1, 2) == 0);
	REQUIRE(AutoRTFM::AlignDown(2, 2) == 2);
	REQUIRE(AutoRTFM::AlignDown(3, 2) == 2);
	REQUIRE(AutoRTFM::AlignDown(4, 2) == 4);
	REQUIRE(AutoRTFM::AlignDown(5, 2) == 4);
	REQUIRE(AutoRTFM::AlignDown(6, 2) == 6);
	REQUIRE(AutoRTFM::AlignDown(7, 2) == 6);
	REQUIRE(AutoRTFM::AlignDown(8, 2) == 8);

	REQUIRE(AutoRTFM::AlignDown(0, 4) == 0);
	REQUIRE(AutoRTFM::AlignDown(1, 4) == 0);
	REQUIRE(AutoRTFM::AlignDown(2, 4) == 0);
	REQUIRE(AutoRTFM::AlignDown(3, 4) == 0);
	REQUIRE(AutoRTFM::AlignDown(4, 4) == 4);
	REQUIRE(AutoRTFM::AlignDown(5, 4) == 4);
	REQUIRE(AutoRTFM::AlignDown(6, 4) == 4);
	REQUIRE(AutoRTFM::AlignDown(7, 4) == 4);
	REQUIRE(AutoRTFM::AlignDown(8, 4) == 8);

	REQUIRE(AutoRTFM::AlignDown(0, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(1, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(2, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(3, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(4, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(5, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(6, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(7, 8) == 0);
	REQUIRE(AutoRTFM::AlignDown(8, 8) == 8);
}

TEST_CASE("AlignUp")
{
	REQUIRE(AutoRTFM::AlignUp(0, 1) == 0);
	REQUIRE(AutoRTFM::AlignUp(1, 1) == 1);
	REQUIRE(AutoRTFM::AlignUp(2, 1) == 2);
	REQUIRE(AutoRTFM::AlignUp(3, 1) == 3);
	REQUIRE(AutoRTFM::AlignUp(4, 1) == 4);
	REQUIRE(AutoRTFM::AlignUp(5, 1) == 5);
	REQUIRE(AutoRTFM::AlignUp(6, 1) == 6);
	REQUIRE(AutoRTFM::AlignUp(7, 1) == 7);
	REQUIRE(AutoRTFM::AlignUp(8, 1) == 8);

	REQUIRE(AutoRTFM::AlignUp(0, 2) == 0);
	REQUIRE(AutoRTFM::AlignUp(1, 2) == 2);
	REQUIRE(AutoRTFM::AlignUp(2, 2) == 2);
	REQUIRE(AutoRTFM::AlignUp(3, 2) == 4);
	REQUIRE(AutoRTFM::AlignUp(4, 2) == 4);
	REQUIRE(AutoRTFM::AlignUp(5, 2) == 6);
	REQUIRE(AutoRTFM::AlignUp(6, 2) == 6);
	REQUIRE(AutoRTFM::AlignUp(7, 2) == 8);
	REQUIRE(AutoRTFM::AlignUp(8, 2) == 8);

	REQUIRE(AutoRTFM::AlignUp(0, 4) == 0);
	REQUIRE(AutoRTFM::AlignUp(1, 4) == 4);
	REQUIRE(AutoRTFM::AlignUp(2, 4) == 4);
	REQUIRE(AutoRTFM::AlignUp(3, 4) == 4);
	REQUIRE(AutoRTFM::AlignUp(4, 4) == 4);
	REQUIRE(AutoRTFM::AlignUp(5, 4) == 8);
	REQUIRE(AutoRTFM::AlignUp(6, 4) == 8);
	REQUIRE(AutoRTFM::AlignUp(7, 4) == 8);
	REQUIRE(AutoRTFM::AlignUp(8, 4) == 8);

	REQUIRE(AutoRTFM::AlignUp(0, 8) == 0);
	REQUIRE(AutoRTFM::AlignUp(1, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(2, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(3, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(4, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(5, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(6, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(7, 8) == 8);
	REQUIRE(AutoRTFM::AlignUp(8, 8) == 8);
}

TEST_CASE("RoundDown")
{
	REQUIRE(AutoRTFM::RoundDown(0, 1) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 1) == 1);
	REQUIRE(AutoRTFM::RoundDown(2, 1) == 2);
	REQUIRE(AutoRTFM::RoundDown(3, 1) == 3);
	REQUIRE(AutoRTFM::RoundDown(4, 1) == 4);
	REQUIRE(AutoRTFM::RoundDown(5, 1) == 5);
	REQUIRE(AutoRTFM::RoundDown(6, 1) == 6);
	REQUIRE(AutoRTFM::RoundDown(7, 1) == 7);
	REQUIRE(AutoRTFM::RoundDown(8, 1) == 8);

	REQUIRE(AutoRTFM::RoundDown(0, 2) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 2) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 2) == 2);
	REQUIRE(AutoRTFM::RoundDown(3, 2) == 2);
	REQUIRE(AutoRTFM::RoundDown(4, 2) == 4);
	REQUIRE(AutoRTFM::RoundDown(5, 2) == 4);
	REQUIRE(AutoRTFM::RoundDown(6, 2) == 6);
	REQUIRE(AutoRTFM::RoundDown(7, 2) == 6);
	REQUIRE(AutoRTFM::RoundDown(8, 2) == 8);

	REQUIRE(AutoRTFM::RoundDown(0, 3) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 3) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 3) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 3) == 3);
	REQUIRE(AutoRTFM::RoundDown(4, 3) == 3);
	REQUIRE(AutoRTFM::RoundDown(5, 3) == 3);
	REQUIRE(AutoRTFM::RoundDown(6, 3) == 6);
	REQUIRE(AutoRTFM::RoundDown(7, 3) == 6);
	REQUIRE(AutoRTFM::RoundDown(8, 3) == 6);

	REQUIRE(AutoRTFM::RoundDown(0, 4) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 4) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 4) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 4) == 0);
	REQUIRE(AutoRTFM::RoundDown(4, 4) == 4);
	REQUIRE(AutoRTFM::RoundDown(5, 4) == 4);
	REQUIRE(AutoRTFM::RoundDown(6, 4) == 4);
	REQUIRE(AutoRTFM::RoundDown(7, 4) == 4);
	REQUIRE(AutoRTFM::RoundDown(8, 4) == 8);

	REQUIRE(AutoRTFM::RoundDown(0, 5) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 5) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 5) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 5) == 0);
	REQUIRE(AutoRTFM::RoundDown(4, 5) == 0);
	REQUIRE(AutoRTFM::RoundDown(5, 5) == 5);
	REQUIRE(AutoRTFM::RoundDown(6, 5) == 5);
	REQUIRE(AutoRTFM::RoundDown(7, 5) == 5);
	REQUIRE(AutoRTFM::RoundDown(8, 5) == 5);

	REQUIRE(AutoRTFM::RoundDown(0, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(4, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(5, 6) == 0);
	REQUIRE(AutoRTFM::RoundDown(6, 6) == 6);
	REQUIRE(AutoRTFM::RoundDown(7, 6) == 6);
	REQUIRE(AutoRTFM::RoundDown(8, 6) == 6);

	REQUIRE(AutoRTFM::RoundDown(0, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(4, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(5, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(6, 7) == 0);
	REQUIRE(AutoRTFM::RoundDown(7, 7) == 7);
	REQUIRE(AutoRTFM::RoundDown(8, 7) == 7);

	REQUIRE(AutoRTFM::RoundDown(0, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(1, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(2, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(3, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(4, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(5, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(6, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(7, 8) == 0);
	REQUIRE(AutoRTFM::RoundDown(8, 8) == 8);
}

TEST_CASE("RoundUp")
{
	REQUIRE(AutoRTFM::RoundUp(0, 1) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 1) == 1);
	REQUIRE(AutoRTFM::RoundUp(2, 1) == 2);
	REQUIRE(AutoRTFM::RoundUp(3, 1) == 3);
	REQUIRE(AutoRTFM::RoundUp(4, 1) == 4);
	REQUIRE(AutoRTFM::RoundUp(5, 1) == 5);
	REQUIRE(AutoRTFM::RoundUp(6, 1) == 6);
	REQUIRE(AutoRTFM::RoundUp(7, 1) == 7);
	REQUIRE(AutoRTFM::RoundUp(8, 1) == 8);

	REQUIRE(AutoRTFM::RoundUp(0, 2) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 2) == 2);
	REQUIRE(AutoRTFM::RoundUp(2, 2) == 2);
	REQUIRE(AutoRTFM::RoundUp(3, 2) == 4);
	REQUIRE(AutoRTFM::RoundUp(4, 2) == 4);
	REQUIRE(AutoRTFM::RoundUp(5, 2) == 6);
	REQUIRE(AutoRTFM::RoundUp(6, 2) == 6);
	REQUIRE(AutoRTFM::RoundUp(7, 2) == 8);
	REQUIRE(AutoRTFM::RoundUp(8, 2) == 8);

	REQUIRE(AutoRTFM::RoundUp(0, 3) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 3) == 3);
	REQUIRE(AutoRTFM::RoundUp(2, 3) == 3);
	REQUIRE(AutoRTFM::RoundUp(3, 3) == 3);
	REQUIRE(AutoRTFM::RoundUp(4, 3) == 6);
	REQUIRE(AutoRTFM::RoundUp(5, 3) == 6);
	REQUIRE(AutoRTFM::RoundUp(6, 3) == 6);
	REQUIRE(AutoRTFM::RoundUp(7, 3) == 9);
	REQUIRE(AutoRTFM::RoundUp(8, 3) == 9);

	REQUIRE(AutoRTFM::RoundUp(0, 4) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 4) == 4);
	REQUIRE(AutoRTFM::RoundUp(2, 4) == 4);
	REQUIRE(AutoRTFM::RoundUp(3, 4) == 4);
	REQUIRE(AutoRTFM::RoundUp(4, 4) == 4);
	REQUIRE(AutoRTFM::RoundUp(5, 4) == 8);
	REQUIRE(AutoRTFM::RoundUp(6, 4) == 8);
	REQUIRE(AutoRTFM::RoundUp(7, 4) == 8);
	REQUIRE(AutoRTFM::RoundUp(8, 4) == 8);

	REQUIRE(AutoRTFM::RoundUp(0, 5) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 5) == 5);
	REQUIRE(AutoRTFM::RoundUp(2, 5) == 5);
	REQUIRE(AutoRTFM::RoundUp(3, 5) == 5);
	REQUIRE(AutoRTFM::RoundUp(4, 5) == 5);
	REQUIRE(AutoRTFM::RoundUp(5, 5) == 5);
	REQUIRE(AutoRTFM::RoundUp(6, 5) == 10);
	REQUIRE(AutoRTFM::RoundUp(7, 5) == 10);
	REQUIRE(AutoRTFM::RoundUp(8, 5) == 10);

	REQUIRE(AutoRTFM::RoundUp(0, 6) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(2, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(3, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(4, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(5, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(6, 6) == 6);
	REQUIRE(AutoRTFM::RoundUp(7, 6) == 12);
	REQUIRE(AutoRTFM::RoundUp(8, 6) == 12);

	REQUIRE(AutoRTFM::RoundUp(0, 7) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(2, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(3, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(4, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(5, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(6, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(7, 7) == 7);
	REQUIRE(AutoRTFM::RoundUp(8, 7) == 14);

	REQUIRE(AutoRTFM::RoundUp(0, 8) == 0);
	REQUIRE(AutoRTFM::RoundUp(1, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(2, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(3, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(4, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(5, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(6, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(7, 8) == 8);
	REQUIRE(AutoRTFM::RoundUp(8, 8) == 8);
}

TEST_CASE("Log2")
{
	static_assert(AutoRTFM::Log2(0u) == 0);
	static_assert(AutoRTFM::Log2(1u) == 0);
	static_assert(AutoRTFM::Log2(2u) == 1);
	static_assert(AutoRTFM::Log2(3u) == 1);
	static_assert(AutoRTFM::Log2(4u) == 2);
	static_assert(AutoRTFM::Log2(5u) == 2);
	static_assert(AutoRTFM::Log2(0x0fffu) == 11);
	static_assert(AutoRTFM::Log2(0x1000u) == 12);
	static_assert(AutoRTFM::Log2(0x1001u) == 12);
	static_assert(AutoRTFM::Log2<uint8_t>(0x0) == 0);
	static_assert(AutoRTFM::Log2<uint8_t>(0xf0) == 7);
	static_assert(AutoRTFM::Log2<uint16_t>(0x0) == 0);
	static_assert(AutoRTFM::Log2<uint16_t>(0xcdef) == 15);
	static_assert(AutoRTFM::Log2<uint32_t>(0x7fffffff) == 30);
	static_assert(AutoRTFM::Log2<uint32_t>(0x80000000) == 31);
	static_assert(AutoRTFM::Log2<uint32_t>(0x80000001) == 31);
	static_assert(AutoRTFM::Log2<uint32_t>(0xfffffffe) == 31);
	static_assert(AutoRTFM::Log2<uint32_t>(0xffffffff) == 31);
	static_assert(AutoRTFM::Log2<uint64_t>(0x7fffffffffffffff) == 62);
	static_assert(AutoRTFM::Log2<uint64_t>(0x8000000000000000) == 63);
	static_assert(AutoRTFM::Log2<uint64_t>(0x8000000000000001) == 63);
	static_assert(AutoRTFM::Log2<uint64_t>(0xfffffffffffffffe) == 63);
	static_assert(AutoRTFM::Log2<uint64_t>(0xffffffffffffffff) == 63);

	for (int Index = 1; Index <= 8; ++Index)
	{
		int LowValue  = (1 << Index) / 2;
		REQUIRE(AutoRTFM::Log2<uint8_t> (LowValue) == Index-1);
		REQUIRE(AutoRTFM::Log2<uint16_t>(LowValue) == Index-1);
		REQUIRE(AutoRTFM::Log2<uint32_t>(LowValue) == Index-1);

		int HighValue = (1 << Index) - 1;
		REQUIRE(AutoRTFM::Log2<uint8_t> (HighValue) == Index-1);
		REQUIRE(AutoRTFM::Log2<uint16_t>(HighValue) == Index-1);
		REQUIRE(AutoRTFM::Log2<uint32_t>(HighValue) == Index-1);
	}
}