// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "WriteLog.h"
#include "Catch2Includes.h"
#include <algorithm>
#include <cstddef>
#include <numeric>
#include <random>
#include <vector>

TEST_CASE("FWriteLog")
{
	std::mt19937 Rand(0x1234);
	AutoRTFM::FWriteLog WriteLog;

	// Returns a vector with the given size filled with random bytes
	auto RandomBuffer = [&Rand](size_t Size) -> std::vector<std::byte>
	{
		std::vector<std::byte> Out(Size);
		for (size_t I = 0; I < Size; I++)
		{
			Out[I] = static_cast<std::byte>(Rand() & 0xff);
		}
		return Out;
	};

	// Returns a std::vector<size_t> with NumEntries elements all equal to
	// EntrySize
	auto FixedSize = [](size_t EntrySize, size_t NumEntries)
	{
		return std::vector<size_t>(NumEntries, EntrySize);
	};

	// Returns a std::vector<size_t> with NumEntries elements with random values
	// between 1 and MaxSize
	auto RandomSize = [&Rand](size_t NumEntries, size_t MaxSize)
	{
		std::vector<size_t> EntrySizes(NumEntries);
		for (size_t& EntrySize : EntrySizes)
		{
			EntrySize = std::uniform_int_distribution<unsigned int>(1, MaxSize)(Rand);
		}
		return EntrySizes;
	};

	// The order in which to add data to the write log.
	enum class EEntryOrder
	{
		Forwards,  // Sequentially forward
		Backwards, // Sequentially backwards
		Random,    // Random order
	};

	auto CheckWithOrder = [&](std::vector<size_t> EntrySizes, EEntryOrder EntryOrder)
	{
		// Total number of write entries to test
		const size_t NumEntries = EntrySizes.size();

		// Total number of bytes to add to the write log
		const size_t TotalBufferSize = std::reduce(EntrySizes.begin(), EntrySizes.end());

		// Buffer of random data
		std::vector<std::byte> Data = RandomBuffer(TotalBufferSize);

		// An interval in Data
		struct FDataSpan
		{
			size_t Offset;
			size_t Size;
		};

		// Produce a list of data spans on Data. These will be added to the write log in order.
		std::vector<FDataSpan> DataSpans(NumEntries);
		{
			size_t Offset = 0;
			for (size_t I = 0; I < NumEntries; I++)
			{
				FDataSpan& DataSpan = DataSpans[I];
				DataSpan.Offset = Offset;
				DataSpan.Size = EntrySizes[I];
				Offset += EntrySizes[I];
			}
			
			// Shuffle the spans
			switch (EntryOrder)
			{
				case EEntryOrder::Forwards:
					break; // Already forwards
				case EEntryOrder::Backwards:
					std::reverse(std::begin(DataSpans), std::end(DataSpans));
					break;
				case EEntryOrder::Random:
					std::shuffle(std::begin(DataSpans), std::end(DataSpans), Rand);
					break;
			}
		}
		
		// Populate the write log and Entries with the expected write log entries
		std::vector<AutoRTFM::FWriteLogEntry> Entries;
		for (FDataSpan& DataSpan : DataSpans)
		{
			AutoRTFM::FWriteLogEntry Entry{};
			Entry.Data = &Data[DataSpan.Offset];
			Entry.Size = DataSpan.Size;
			Entry.LogicalAddress = reinterpret_cast<std::byte*>(static_cast<uintptr_t>(0x1234000 + DataSpan.Offset));
			if ((Rand() & 3) == 0) // 25% of write log entries can have non-default flags.
			{
				if (Rand() & 1)
				{
					Entry.Flags = Entry.Flags | AutoRTFM::EWriteFlags::NoSanitize;
				}
				if (Rand() & 1)
				{
					Entry.Flags = Entry.Flags | AutoRTFM::EWriteFlags::CustomRollback;
					if (Rand() & 1)
					{
						Entry.Flags = Entry.Flags | AutoRTFM::EWriteFlags::CustomFlag0;
					}
					if (Rand() & 1)
					{
						Entry.Flags = Entry.Flags | AutoRTFM::EWriteFlags::CustomFlag1;
					}
				}
			}
			Entries.push_back(Entry);
			WriteLog.Push(Entry);
		}

		REQUIRE(!WriteLog.IsEmpty());
		REQUIRE(WriteLog.TotalSize() == Data.size());

		SECTION("Forwards iterator")
		{
			AutoRTFM::FWriteLogEntry Expect;
			ptrdiff_t EntryIndex = 0;
			for (AutoRTFM::FWriteLog::Iterator It = WriteLog.begin(); It != WriteLog.end(); ++It)
			{
				AutoRTFM::FWriteLogEntry Got = *It;
				while (Got.Size > 0)
				{
					// Move to the next expectation record when this one is fully empty.
					if (Expect.Size == 0)
					{
						Expect = Entries[EntryIndex++];
					}

					// The actual write log must match the expectation's address and flags.
					REQUIRE(Got.LogicalAddress == Expect.LogicalAddress);
					REQUIRE(Got.Flags == Expect.Flags);

					// Compare the actual contents of the write log with our expectations.
					const size_t NumBytes = std::min(Got.Size, Expect.Size);
					REQUIRE(memcmp(Got.Data, Expect.Data, NumBytes) == 0);
					Got.LogicalAddress += NumBytes;
					Got.Data += NumBytes;
					Got.Size -= NumBytes;
					Expect.LogicalAddress += NumBytes;
					Expect.Data += NumBytes;
					Expect.Size -= NumBytes;
				}
				REQUIRE(Got.Size == 0);
			}
			REQUIRE(EntryIndex == Entries.size());
		}

		SECTION("Reverse iterator")
		{
			AutoRTFM::FWriteLogEntry Expect;
			ptrdiff_t EntryIndex = Entries.size();
			for (AutoRTFM::FWriteLog::ReverseIterator It = WriteLog.rbegin(); It != WriteLog.rend(); ++It)
			{
				AutoRTFM::FWriteLogEntry Got = *It;
				while (Got.Size > 0)
				{
					// Move to the previous expectation record when this one is fully empty.
					if (Expect.Size == 0)
					{
						Expect = Entries[--EntryIndex];
					}

					// The actual write log must match the expectation's address and flags.
					REQUIRE(Got.LogicalAddress + Got.Size == Expect.LogicalAddress + Expect.Size);
					REQUIRE(Got.Flags == Expect.Flags);

					// Compare the actual contents of the write log with our expectations.
					// Since we are reading the lists in reverse, we check the right edge of the data, and 
					// don't need to increment pointers.
					const size_t NumBytes = std::min(Got.Size, Expect.Size);
					REQUIRE(memcmp(Got.Data + Got.Size - NumBytes, Expect.Data + Expect.Size - NumBytes, NumBytes) == 0);
					Got.Size -= NumBytes;
					Expect.Size -= NumBytes;
				}
				REQUIRE(Got.Size == 0);
			}
			REQUIRE(EntryIndex == 0);
		}

		SECTION("Reset")
		{
			WriteLog.Reset();
			REQUIRE(WriteLog.IsEmpty());
			REQUIRE(WriteLog.Num() == 0u);
			REQUIRE(WriteLog.TotalSize() == 0u);
		}
	};
	
	auto Check = [&](std::vector<size_t> EntrySizes)
	{
		SECTION("Forwards")
		{
			CheckWithOrder(std::move(EntrySizes), EEntryOrder::Forwards);
		}
		SECTION("Backwards")
		{
			CheckWithOrder(std::move(EntrySizes), EEntryOrder::Backwards);
		}
		SECTION("Random")
		{
			CheckWithOrder(std::move(EntrySizes), EEntryOrder::Random);
		}
	};

	SECTION("Empty")
	{
		REQUIRE(WriteLog.IsEmpty());
		REQUIRE(WriteLog.Num() == 0u);
		REQUIRE(WriteLog.TotalSize() == 0u);
	}

	SECTION("EntrySize: 1, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ 1, /* NumEntries */ 32));
	}

	SECTION("EntrySize: 32, NumEntries: 65536")
	{
		Check(FixedSize(/* EntrySize */ 32, /* NumEntries */ 65536));
	}

	SECTION("EntrySize: 1024, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ 1024, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FRegularRecordLayout::MaxSize-1, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FRegularRecordLayout::MaxSize - 1, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FRegularRecordLayout::MaxSize, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FRegularRecordLayout::MaxSize, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FRegularRecordLayout::MaxSize+1, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FRegularRecordLayout::MaxSize + 1, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FRegularRecordLayout::MaxSize*10, NumEntries: 10")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FRegularRecordLayout::MaxSize * 10, /* NumEntries */ 10));
	}

	SECTION("EntrySize: FCustomRollbackRecordLayout::MaxSize-1, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FCustomRollbackRecordLayout::MaxSize - 1, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FCustomRollbackRecordLayout::MaxSize, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FCustomRollbackRecordLayout::MaxSize, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FCustomRollbackRecordLayout::MaxSize+1, NumEntries: 32")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FCustomRollbackRecordLayout::MaxSize + 1, /* NumEntries */ 32));
	}

	SECTION("EntrySize: FCustomRollbackRecordLayout::MaxSize*10, NumEntries: 10")
	{
		Check(FixedSize(/* EntrySize */ AutoRTFM::FWriteLog::FCustomRollbackRecordLayout::MaxSize * 10, /* NumEntries */ 10));
	}

	SECTION("EntrySize: random, NumEntries: 32")
	{
		Check(RandomSize(/* NumEntries */ 32, /* MaxSize */ 0x8000));
	}
}
